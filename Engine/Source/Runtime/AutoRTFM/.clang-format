---
AccessModifierOffset: -4
AlignAfterOpenBracket: DontAlign
AlignArrayOfStructures: Right
AlignConsecutiveAssignments: None
AlignConsecutiveBitFields: None
AlignConsecutiveDeclarations: None
AlignEscapedNewlines: Left
AlignOperands: AlignAfterOperator
AlignTrailingComments: true
AllowShortBlocksOnASingleLine: Empty
AllowShortEnumsOnASingleLine: false
AllowShortFunctionsOnASingleLine: Empty
AllowShortLambdasOnASingleLine: Inline
AlwaysBreakTemplateDeclarations: true # should become BreakTemplateDeclarations: Yes when clang-format is updated
BasedOnStyle: LLVM
BreakBeforeBinaryOperators: NonAssignment
BreakBeforeBraces: Allman
BreakBeforeTernaryOperators: true
BreakConstructorInitializers: BeforeComma
BreakInheritanceList: BeforeComma
BreakStringLiterals: true
ColumnLimit: 133
Cpp11BracedListStyle: true
EmptyLineBeforeAccessModifier: LogicalBlock
FixNamespaceComments: true
ShortNamespaceLines: 30
IncludeBlocks: Preserve
IndentCaseBlocks: false
IndentCaseLabels: true
IndentExternBlock: NoIndent
IndentPPDirectives: None
IndentRequiresClause: false
IndentWidth: 4
Language: Cpp
NamespaceIndentation: None
PackConstructorInitializers: CurrentLine
PointerAlignment: Left
ReferenceAlignment: Left
RemoveSemicolon: true
RequiresClausePosition: OwnLine
SortIncludes: true
SpaceBeforeCaseColon: false
SpacesBeforeTrailingComments: 2
Standard: c++20
TabWidth: 4
UseTab: ForContinuationAndIndentation
