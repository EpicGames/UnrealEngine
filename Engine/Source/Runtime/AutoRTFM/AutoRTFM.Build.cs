// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AutoRTFM : ModuleRules
	{
		public AutoRTFM(ReadOnlyTargetRules Target) : base(Target)
		{
			// Header-only dependency on Core so that we can include HAL/Platform.h
			// to have a definition for DLLEXPORT / DLLIMPORT.
			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"Core"
				}
			);
			PrivateDefinitions.Add("SUPPRESS_PER_MODULE_INLINE_FILE"); // This module does not use core's standard operator new/delete overloads

			AutoRTFMExternalMappingFiles.Add("Runtime/AutoRTFM/Public/Internal.aem");
			AutoRTFMExternalMappingFiles.Add("Runtime/AutoRTFM/Public/StdLib.Common.aem");
			
			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				AutoRTFMExternalMappingFiles.Add("Runtime/AutoRTFM/Public/StdLib.Windows.aem");

				if (Target.WindowsPlatform.bEnableAddressSanitizer)
				{
					AutoRTFMExternalMappingFiles.Add("Runtime/AutoRTFM/Public/ASAN.aem");
				}

				if (Target.WindowsPlatform.bEnableUndefinedBehaviorSanitizer)
				{
					AutoRTFMExternalMappingFiles.Add("Runtime/AutoRTFM/Public/UBSAN.aem");
				}
				
				if (Target.StaticAllocator == StaticAllocatorType.Ansi)
				{
					AutoRTFMExternalMappingFiles.Add("Runtime/AutoRTFM/Public/NewDelete.Windows.aem");
				}
			}
			if (Target.Platform == UnrealTargetPlatform.Linux || Target.Platform == UnrealTargetPlatform.LinuxArm64)
			{
				AutoRTFMExternalMappingFiles.Add("Runtime/AutoRTFM/Public/StdLib.Linux.aem");

				if (Target.LinuxPlatform.bEnableAddressSanitizer)
				{
					AutoRTFMExternalMappingFiles.Add("Runtime/AutoRTFM/Public/ASAN.aem");
				}

				if (Target.LinuxPlatform.bEnableUndefinedBehaviorSanitizer)
				{
					AutoRTFMExternalMappingFiles.Add("Runtime/AutoRTFM/Public/UBSAN.aem");
				}

				if (Target.LinuxPlatform.bEnableMemorySanitizer)
				{
					AutoRTFMExternalMappingFiles.Add("Runtime/AutoRTFM/Public/MSAN.aem");
				}

				if (Target.StaticAllocator == StaticAllocatorType.Ansi)
				{
					AutoRTFMExternalMappingFiles.Add("Runtime/AutoRTFM/Public/NewDelete.Linux.aem");
				}
			}

			// Enable (almost) all warnings as errors
			CppCompileWarningSettings.ShadowVariableWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.UndefinedIdentifierWarningLevel = WarningLevel.Error;
			// CppCompileWarningSettings.SwitchUnhandledEnumeratorWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.EnumEnumConversionWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.EnumFloatConversionWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.AmbiguousReversedOperatorWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.DeprecatedAnonEnumEnumConversionWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.DeprecatedVolatileWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.OrderedCompareFunctionPointers = WarningLevel.Error;
			CppCompileWarningSettings.BitwiseInsteadOfLogicalWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.DeprecatedCopyWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.DeprecatedCopyWithUserProvidedCopyWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.InvalidUnevaluatedStringWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.NaNInfinityDisabledWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.LevelExtraQualificationWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.CastFunctionTypeMismatchWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.MissingTemplateArgListAfterTemplateWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.GNUStringLiteralOperatorTemplateWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.InconsistentMissingOverrideWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.InvalidOffsetWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.SwitchWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.TautologicalCompareWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.UnknownPragmasWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.UnusedWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.UndefinedVarTemplateWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.ProfileInstructWarningLevel = WarningLevel.Error;
			// CppCompileWarningSettings.BackendPluginWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.UnusedValueWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.Shorten64To32WarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.DllExportExplicitInstantiationDeclWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.MicrosoftGroupWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.MSVCIncludeWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.PragmaPackWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.InlineNewDeleteWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.ImplicitExceptionSpecMismatchWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.UndefinedBoolConversionWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.DeprecatedWriteableStringsWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.DeprecatedRegisterWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.LogicalOpParenthesesWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.NullArithmeticWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.ReturnTypeCLinkageWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.IgnoredAttributesWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.UninitializedWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.ReturnTypeWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.UnusedParameterWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.IgnoredQualifiersWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.ExpansionToDefined = WarningLevel.Error;
			CppCompileWarningSettings.SignCompareWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.MissingFieldInitializersWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.NonPortableIncludePathWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.InvalidTokenPasteWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.NullPointerArithmeticWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.ConstantLogicalOperandWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.BitfieldEnumConversion = WarningLevel.Error;
			CppCompileWarningSettings.NullPointerSubtractionWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.DanglingWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.MSVCSwitchEnumWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.MSVCUnusedValueWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.MSVCDeprecationWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.DeprecationWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.DeterministicWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.PCHPerformanceIssueWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.ModuleUnsupportedWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.PluginModuleUnsupportedWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.ModuleIncludePathWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.ModuleIncludePrivateWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.ModuleIncludeSubdirectoryWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.DisablePluginsConflictWarningLevel = WarningLevel.Error;
			CppCompileWarningSettings.NonInlinedGenCppWarningLevel = WarningLevel.Error;
		}
	}
}
