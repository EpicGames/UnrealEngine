<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!-- Epic Games, Inc. AutoRTFM Visualizers -->

  <Type Name="AutoRTFM::FWriteLog">
    <DisplayString>Count: {NumEntries}</DisplayString>
    <Expand>
      <Item Name="Count">NumEntries</Item>
      <Item Name="TotalSizeBytes">TotalSizeBytes</Item>
      <Synthetic Name="Writes">
        <Expand>
          <CustomListItems>
            <Variable Name="Block" InitialValue="HeadBlock" />
            <Variable Name="Record" InitialValue="(*Block).FirstRecord" />
            <Variable Name="Data" InitialValue="(std::byte*)(Block) + sizeof(AutoRTFM::FWriteLog::FBlock)" />
            <Variable Name="Index" InitialValue="0" />
            <Loop>
              <Loop Condition="(Block != nullptr) &amp;&amp; (Record &lt; (*Block).LastRecord)">
                <Exec>Block = (*Block).NextBlock</Exec>
                <Exec>Record = (*Block).FirstRecord</Exec>
                <Exec>Data = (std::byte*)(Block) + sizeof(AutoRTFM::FWriteLog::FBlock)</Exec>
              </Loop>
              <Break Condition="Block == nullptr" />
              <Item Name="[{Index}] Record">*Record,x</Item>
              <If Condition="(*Record).Size == 2">
                <Item Name="[{Index}] Data">(uint32*)(Data),x</Item>
              </If>
              <Else>
                <If Condition="(*Record).Size == 4">
                  <Item Name="[{Index}] Data">(uint32*)(Data),x</Item>
                </If>
                <Else>
                  <If Condition="(*Record).Size == 8">
                    <Item Name="[{Index}] Data">(uint64*)(Data),x</Item>
                  </If>
                  <Else>
                    <Item Name="[{Index}] Data">Data,[(*Record).Size]</Item>
                  </Else>
                </Else>
              </Else>
              <Exec>Index++</Exec>
              <Exec>Data += (*Record).Size</Exec>
              <Exec>Record--</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="Blocks">
        <Expand>
          <CustomListItems>
            <Variable Name="Block" InitialValue="HeadBlock" />
            <Variable Name="Index" InitialValue="0" />
            <Loop Condition="Block != nullptr">
              <Item Name="[{Index}]">*Block</Item>
              <Exec>Index++</Exec>
              <Exec>Block = (*Block).NextBlock</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

  <Type Name="AutoRTFM::FWriteLog::FBlock">
    <DisplayString>Count: {FirstRecord - LastRecord + 1}</DisplayString>
    <Expand>
      <Item Name="Count">FirstRecord - LastRecord + 1</Item>
      <Item Name="DataSize">DataEnd - ((std::byte*)(this) + sizeof(AutoRTFM::FWriteLog::FBlock))</Item>
      <Item Name="BlockSize">(std::byte*)(FirstRecord + 1) - (std::byte*)(this)</Item>
      <CustomListItems>
        <Variable Name="Record" InitialValue="FirstRecord" />
        <Variable Name="Data" InitialValue="(std::byte*)(this) + sizeof(AutoRTFM::FWriteLog::FBlock)" />
        <Variable Name="Index" InitialValue="0" />
        <Loop Condition="Record &gt;= LastRecord">
          <Item Name="[{Index}] Record">*Record,x</Item>
          <If Condition="(*Record).Size() == 2">
              <Item Name="[{Index}] Data">(uint32*)(Data),x</Item>
          </If>
          <Else>
            <If Condition="(*Record).Size() == 4">
                <Item Name="[{Index}] Data">(uint32*)(Data),x</Item>
            </If>
            <Else>
              <If Condition="(*Record).Size() == 8">
                <Item Name="[{Index}] Data">(uint64*)(Data),x</Item>
              </If>
              <Else>
                <Item Name="[{Index}] Data">Data,[(*Record).Size()]</Item>
              </Else>
            </Else>
          </Else>
          <Exec>Index++</Exec>
          <Exec>Data += (*Record).Size()</Exec>
          <Exec>Record--</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>

  <Type Name="AutoRTFM::FWriteLog::FRecord">
    <Intrinsic Name="IsCustomWrite" Expression="(*this).Bits &amp; 0x2000000000000" />
    <Intrinsic Name="BitRange" Expression="((*this).Bits &gt;&gt; First) &amp; (((uint64_t)(1) &lt;&lt; (Last - First)) - 1)">
      <Parameter Name="First" Type="uint64_t" />
      <Parameter Name="Last" Type="uint64_t" />
    </Intrinsic>
    <Intrinsic Name="Address" Expression="(void*)(BitRange(0, 48))" />
    <Intrinsic Name="Flags" Expression="(autortfm_write_flags)(BitRange(48, IsCustomWrite() ? 52 : 50))" />
    <Intrinsic Name="Size" Expression="BitRange(IsCustomWrite() ? 52 : 50, 64) + 1" />
    <Expand>
      <Item Name="Address">Address()</Item>
      <Item Name="Flags">Flags()</Item>
      <Item Name="Size">Size()</Item>
    </Expand>
  </Type>

  <Type Name="AutoRTFM::TStack&lt;*&gt;">
    <Expand>
      <IndexListItems>
        <Size>(*this).Count</Size>
        <ValueNode>(*this).Data[$i]</ValueNode>
      </IndexListItems>
    </Expand>
  </Type>

</AutoVisualizer>
