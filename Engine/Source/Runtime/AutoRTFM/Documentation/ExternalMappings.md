# AutoRTFM External Mapping Files

## About

AutoRTFM External Mapping Files are files consumed by the `verse-clang` compiler and control the how the compiler maps "open" (uninstrumented functions) to their "closed" (an AutoRTFM-instrumented function) variant, when the function is called in the closed. These files provide a way to control the AutoRTFM behavior on functions declared in source that cannot be modified (e.g. standard library code, third-party code, pre-compiled libraries, etc).

AutoRTFM External Mapping Files have the extension `.aem`.

## Syntax

Line comments begin with a `#`.

Statements:

* `map` **[open-function]** `->` **[closed-function]**\
    Calls to **[open-function]** will be mapped to **[closed-function]** when called from a closed function.\
    **[open-function]** must not have any visible definition (body) when compiling with AutoRTFM instrumentation enabled.

* `same` **[function]**\
    Shorthand for `map` **[function]** `->` **[function]**

* `disable` **[function]**\
    Disables AutoRTFM instrumentation on **[function]**. Attempting to call this from a closed function will result in a compile-time error.

Function names use their mangled form.

## UnrealBuildTool integration

Each target's `.Build.cs` file can associate AutoRTFM External Mapping Files with the target by appending to the module's `AutoRTFMExternalMappingFiles` list. For example:

```c#
	AutoRTFMExternalMappingFiles.Add("Path/To/My/Mappings.aem");
```

The module and all modules that transitively depend on the module will be compiled using the mapping file.
Note that modifying an AutoRTFM External Mapping File will cause all modules that depend on the file to be rebuilt in their entirety.
