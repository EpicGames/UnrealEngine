# AutoRTFM modes and attributes

When compiling a module using the AutoRTFM compiler with instrumentation enabled,
every function is compiled with a single, mutually-exclusive [AutoRTFM mode](#autortfm-modes).

The AutoRTFM mode controls the behavior of the [closed variant](README.md#open-and-closed-code) of the function.

AutoRTFM modes are specified using [AutoRTFM attributes](#mode-attributes) on lambdas, functions, methods and classes.

The UBT module flag `bInjectAutoRTFMAttributeDisables` can also be used to apply [`AUTORTFM_DISABLE`](#autortfm-disabled) to all module exports (e.g. `FOO_API`).

## AutoRTFM modes

Every function (including lambdas, operators, methods, etc) compiled with AutoRTFM
instrumentation has an AutoRTFM mode. This is one of the following:

### AutoRTFM-enabled

AutoRTFM instrumentation **is** generated for the function. This is the default
mode for targets compiled with AutoRTFM instrumentation enabled.

An AutoRTFM-enabled function is callable from the **open** and the **closed**.

Represented by the enum `autortfm_mode_enable`.

### AutoRTFM-disabled

AutoRTFM instrumentation **is not** generated for the function.
This function is callable from the **open**, but not the **closed**.

Attempting to call an AutoRTFM-disabled function from an
[AutoRTFM-enabled](#autortfm-enabled) function is a compile-time error, unless
the call is unconditionally preceded with a call to
`AutoRTFM::UnreachableIfClosed()` or `AutoRTFM::UnreachableIfTransactional()`.

Use of `AutoRTFM::UnreachableIfClosed()` / `AutoRTFM::UnreachableIfTransactional()`
will allow code to compile that calls AutoRTFM-disabled functions from
[AutoRTFM-enabled](#autortfm-enabled) functions, but will trigger a runtime
assertion if called from the closed.

Represented by the enum `autortfm_mode_disable`.

### AutoRTFM-inferred

AutoRTFM instrumentation **may be** generated for the function, so long as the
function does not call any other function that is
[AutoRTFM-disabled](#autortfm-disabled) directly, or transitively via other
AutoRTFM-inferred functions.

This mode is most commonly used on templated classes or functions that call
template-dependent functions which may have different AutoRTFM modes. This
allows the template to mimic the AutoRTFM-enabled or AutoRTFM-disabled mode of
a template-dependent callee.

Attempting to call an AutoRTFM-inferred function from an
[AutoRTFM-enabled](#autortfm-enabled) function is a compile-time error if the
AutoRTFM-inferred function makes a direct call to an
[AutoRTFM-disabled](#autortfm-disabled) function, or transitively via other
AutoRTFM-inferred functions.

Similarly to [AutoRTFM-disabled](#autortfm-disabled) functions, use of
`AutoRTFM::UnreachableIfClosed()` or `AutoRTFM::UnreachableIfTransactional()`
can be used to allow code to compile that would otherwise fail, but with the
same caveat that this may cause runtime assertions.

Represented by the enum `autortfm_mode_infer`.

### AutoRTFM-open

AutoRTFM instrumentation **is not** generated for the function. Instead a stub
**closed** function is generated that will:

- Transition the transaction state from the **closed** to the **open**.
- Call the uninstrumented open function.
- Transition the transaction state back to the **closed** before returning.

This function is callable from the **open** and the **closed**.

Represented by the enum `autortfm_mode_open`.

### AutoRTFM-open-with-no-sanitize

Behaves the same as [AutoRTFM-open](#autortfm-open), but disables the
AutoRTFM sanitizer for calls to this function.

Represented by the enum `autortfm_mode_open_no_sanitize`.

### AutoRTFM-internal

Exclusively used by the AutoRTFM runtime. Not to be used externally.

Calling the function in **the closed** calls the uninstrumented open function
without a stub function to transition the transaction state to the open.

Represented by the enum `autortfm_mode_internal`.

## Mode Attributes

The following attributes can be applied to lambdas, functions, classes and structs:

| Attribute                               | Mode |
|-----------------------------------------|------|
| `AUTORTFM_ENABLE`                       | Unconditionally applies the [AutoRTFM-enabled](#autortfm-enabled) mode. |
| `AUTORTFM_ENABLE_IF(<cond>)`            | Conditionally applies the [AutoRTFM-enabled](#autortfm-enabled) mode if `<cond>` evaluates to true, otherwise behaves as if the attribute did not exist. |
| `AUTORTFM_DISABLE`                      | Unconditionally applies the [AutoRTFM-disabled](#autortfm-disabled) mode. |
| `AUTORTFM_DISABLE_IF(<cond>)`           | Conditionally applies the [AutoRTFM-disabled](#autortfm-disabled) mode if `<cond>` evaluates to true, otherwise behaves as if the attribute did not exist. |
| `AUTORTFM_OPEN`                         | Unconditionally applies the [AutoRTFM-open](#autortfm-open) mode. |
| `AUTORTFM_OPEN_NO_SANITIZE`             | Unconditionally applies the [AutoRTFM-open-with-no-sanitize](#autortfm-open-with-no-sanitize) mode. |
| `AUTORTFM_INFER`                        | Unconditionally applies the [AutoRTFM-inferred](#autortfm-inferred) mode. |

Attributes on functions and methods need to be placed on the declaration (usually in the `.h`) and positioned before the return type (C-declaration style).\
Example: `AUTORTFM_DISABLE void Foo();`

Attributes on lambdas need to be placed after the captures but before parameters or body.\
Example with parameters: `auto Lambda = [] AUTORTFM_DISABLE (int Param) {};`\
Example without parameters: `auto Lambda = [] AUTORTFM_DISABLE {};`

Attributes on classes and structs need to be positioned before the type's name.\
Example: `class AUTORTFM_OPEN MyClass { /* ... */ };`

The following attributes can be applied to function parameters:

| Attribute                               | Mode |
|-----------------------------------------|------|
| `AUTORTFM_IMPLICIT_ENABLE`              | When a "naked" lambda is passed as an argument to this parameter, the lambda's mode will be [enabled](#autortfm-enabled), regardless of outer function or class attributes. |
| `AUTORTFM_IMPLICIT_DISABLE`             | When a "naked" lambda is passed as an argument to this parameter, the lambda's mode will be [disabled](#autortfm-disabled), regardless of outer function or class attributes. |

## Mode Querying

It is possible to query the [AutoRTFM mode](#autortfm-modes) of a function, function overload, `class` or `struct` using the following APIs:

### `AUTORTFM_MODE_OF(EXPR_OR_TYPE)`

Returns the [AutoRTFM mode](#autortfm-modes) of a function by address, method by address or `class` or `struct`. Examples:

```C++
AUTORTFM_ENABLE void MyEnabledFunction();
static_assert(AUTORTFM_MODE_OF(&MyEnabledFunction) == autortfm_mode_enable);
```

```C++
class AUTORTFM_DISABLE MyDisabledClass {};
static_assert(AUTORTFM_MODE_OF(MyDisabledClass) == autortfm_mode_disable);
```

Note: Currently the [AutoRTFM mode](#autortfm-modes) is not a property of a function pointer type, so `AUTORTFM_MODE_OF` may give incorrect results if passed an indirect function pointer (say via a variable).

### `AUTORTFM_MODE_OF_CALL(CALL_EXPR)`

Returns the [AutoRTFM mode](#autortfm-modes) of a function by call expression, which can be used to disambiguate overloads.

```C++
AUTORTFM_ENABLE void MyOverload(int);
AUTORTFM_DISABLE void MyOverload(void*);
static_assert(AUTORTFM_MODE_OF_CALL(MyOverload(42)) == autortfm_mode_enable);
static_assert(AUTORTFM_MODE_OF_CALL(MyOverload(nullptr)) == autortfm_mode_disable);
```

### `AutoRTFM::CallMode<FunctorType, ArgTypes...>`

Evaluates to the [AutoRTFM mode](#autortfm-modes) of a functor type (type
with `operator()` method) with the given argument types.

```C++
struct MyFunctor
{
    AUTORTFM_ENABLE  bool operator()(int) const;
    AUTORTFM_DISABLE bool operator()(bool) const;
};

static_assert(AutoRTFM::CallMode<MyFunctor, int> == autortfm_mode_enable);
static_assert(AutoRTFM::CallMode<MyFunctor, bool> == autortfm_mode_disable);
```

### `AutoRTFM::DestructorMode<Type>`

Evaluates to the [AutoRTFM mode](#autortfm-modes) of the destructor method
of `Type`.

```C++
struct DisabledDestructor
{
    AUTORTFM_DISABLE ~DisabledDestructor();
};
static_assert(AutoRTFM::DestructorMode<DisabledDestructor> == autortfm_mode_disable);
```

### `AutoRTFM::ConstructorMode<Type, ArgTypes...>`

Evaluates to the [AutoRTFM mode](#autortfm-modes) of the constructor method
of `Type` when called with `ArgTypes`.

```C++
struct OverloadedConstructors
{
    AUTORTFM_DISABLE OverloadedConstructors();
    AUTORTFM_ENABLE  OverloadedConstructors(int);
};
static_assert(AutoRTFM::ConstructorMode<OverloadedConstructors> == autortfm_mode_disable);
static_assert(AutoRTFM::ConstructorMode<OverloadedConstructors, int> == autortfm_mode_enable);
```

## Mode Attribute Rules

The mode used is determined by the following rules, starting with the highest precedence:

1. Legacy attributes (`UE_AUTORTFM_ALWAYS_OPEN`, `UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION`,
   `UE_AUTORTFM_NOAUTORTFM`) on the function itself take highest priority.
   These do **not** propagate to nested lambdas or local functions.
2. A lambda, function, class or struct directly annotated with any of the
   unconditional or conditionally-true [AutoRTFM mode attributes](#mode-attributes)
   will use that attribute's mode.
3. A "naked" (passed directly, not via a variable) lambda, with no explicit
   AutoRTFM mode attribute, passed to a function parameter annotated with the
   `AUTORTFM_IMPLICIT_ENABLE` or `AUTORTFM_IMPLICIT_DISABLE` attributes will be
   [enabled](#autortfm-enabled) or [disabled](#autortfm-disabled), respectively. \
   For example the lambda passed to `AutoRTFM::Transact([] { /* ... */ })` is
   not explicitly annotated, and the single `AutoRTFM::Transact()` parameter
   is annotated with `AUTORTFM_IMPLICIT_ENABLE`, so the lambda will be [enabled](#autortfm-enabled),
   regardless of the outer function or class attributes.
4. A function scoped lambda, or function scoped class will use the AutoRTFM mode
   of the parent function.
5. An unannotated (or annotated with an conditionally-false [AutoRTFM mode attribute](#mode-attributes))
   `class` or `struct` method will use the mode of the parent `class` or `struct`.
6. An unannotated (or annotated with an conditionally-false [AutoRTFM mode attribute](#mode-attributes))
   nested `class` or `struct` will use the mode of the parent `class` or `struct`.
7. An unannotated function in a **template context** — i.e. a function template
   specialization, a member of a class template specialization, a function in
   the `std::` namespace, or a local function/struct defined inside any of these
   — will default to [AutoRTFM-inferred](#autortfm-inferred) if the function has
   a visible body. This allows template instantiations to adapt their mode to the
   types they are instantiated with. For example, `std::vector<DisabledType>::push_back()`
   will infer to [AutoRTFM-disabled](#autortfm-disabled) because it calls the
   disabled copy/move constructor. \
   Note: Functions with explicit AutoRTFM attributes (rules 1–2) are never
   affected by this heuristic. AutoRTFM runtime functions that intentionally
   accept disabled functors (e.g. `AutoRTFM::Open`, `AutoRTFM::OnCommit`) must
   be explicitly annotated with `AUTORTFM_ENABLE` to prevent inference. \
   Note: `extern template` declarations and `dllimport` functions are excluded
   from this heuristic because their bodies are not available for analysis.
8. An unnested `class` or `struct` will be [AutoRTFM-enabled](#autortfm-enabled)
   unless the module has instrumentation disabled.

Perhaps a simpler way to describe these rules is:
> With exception of the `AUTORTFM_IMPLICIT_ENABLE` and `AUTORTFM_IMPLICIT_DISABLE`
parameter attributes, **the AutoRTFM mode used by a function is the mode taken
from the closest enclosing scope with an unconditional or conditionally-true
AutoRTFM attribute**. If no such attribute exists and the function is in a
template context, it defaults to [AutoRTFM-inferred](#autortfm-inferred).

### Examples

```C++
struct AUTORTFM_DISABLE DisabledStruct // AutoRTFM-disabled
{
    // AUTORTFM_DISABLE will apply to any method, nested class / structs that
    // do not have their own more-specific AutoRTFM mode attribute.
    void DisabledByAttributeOnStruct(); // AutoRTFM-disabled

    AUTORTFM_ENABLE void EnabledByExplicitAttribute(); // AutoRTFM-enabled

    class DisabledByOuterStruct { /* ... */ }; // AutoRTFM-disabled

    class AUTORTFM_OPEN OpenByExplicitAttribute // AutoRTFM-open
    {
        void OpenByAttributeOnClass(); // AutoRTFM-open

        AUTORTFM_DISABLE void DisabledByExplicitAttribute();  // AutoRTFM-disabled
    };
};

void DisabledStruct::DisabledByAttributeOnStruct()
{
    // Lambda that is AutoRTFM-disabled by the AUTORTFM_DISABLE attribute on DisabledStruct.
    auto LambdaDisabledByAttributeOnStruct = []{ /* ... */ };

    // Lambda that is AutoRTFM-open by the explicit AUTORTFM_OPEN attribute on the lambda.
    auto ExplicitOpenLambda = [] AUTORTFM_OPEN { /* ... */ };
}

void DisabledStruct::EnabledByExplicitAttribute()
{
    if (SomeConditional())
    {
        // UnreachableIfClosed() will runtime-error if this branch is taken in the closed.
        AutoRTFM::UnreachableIfClosed("I promise that this is not reachable in the closed");
        // The UnreachableIfClosed() above prevents this call to an AutoRTFM-disabled
        // function from an AutoRTFM-enabled function from being a compile-time error.
        DisabledByAttributeOnStruct();
    }

    // Attempting to call an AutoRTFM-disabled function from this AutoRTFM-enabled
    // function, without a preceding call to AutoRTFM::UnreachableIfClosed(), is a
    // compile-time error:
    //   error: Cannot call AutoRTFM-disabled function from an AutoRTFM-enabled function.
    //   note: AutoRTFM-disabled function 'DisabledStruct::DisabledByAttributeOnStruct()'
    //   note: called by AutoRTFM-enabled function 'DisabledStruct::EnabledByExplicitAttribute()'
    DisabledByAttributeOnStruct();
}
```

```C++
template<typename FunctorType>
void CallClosedFunctor(AUTORTFM_IMPLICIT_ENABLE FunctorType&&) { /* ... */ }

AUTORTFM_DISABLE void DisabledFunction()
{
    CallClosedFunctor([]
    {
        // Despite the outer function being annotated with AUTORTFM_DISABLE,
        // this lambda will be AutoRTFM-enabled due to the lambda being passed
        // directly to a AUTORTFM_IMPLICIT_ENABLE annotated parameter.
    });

    // This is not a direct passing of the lambda to the AUTORTFM_IMPLICIT_ENABLE
    // annotated parameter, and so DisabledLambda will be AutoRTFM-disabled due
    // to the AUTORTFM_DISABLE attribute on DisabledFunction.
    auto DisabledLambda = []{};
    CallClosedFunctor(DisabledLambda);
}
```

## FAQ

### The compiler errors with `error: Cannot call AutoRTFM-disabled function from an AutoRTFM-enabled function`

You are attempting to compile an [AutoRTFM-enabled](#autortfm-enabled) function
that is making a call to either:

- An [AutoRTFM-disabled](#autortfm-disabled) function, or
- An [AutoRTFM-inferred](#autortfm-inferred) function which transitively calls
  an [AutoRTFM-disabled](#autortfm-disabled) function.

When the callee was inferred to disable, the diagnostic will show the full
chain of calls that caused the inference, making it easier to identify the
root cause.

The compiler is erroring because the callee function has no closed variant for
the closed caller function to call.\
To fix this, you have a few choices:

1. If the caller function is not expected to be reachable in the closed, apply
   [`AUTORTFM_DISABLE`](#autortfm-disabled) to this function (or owning class if
   this is a method and the entire class should not be callable from the closed).
   This option is ideal for reducing code size bloat, improving compilation
   performance, and alerting developers to AutoRTFM mistakes, but
   `AUTORTFM_DISABLE` is "viral" and may cause callers of the disabled function to
   start erroring with a similar message until all of those are also fixed up.\
   Use of `AUTORTFM_DISABLE` for these errors is the preferred long-term solution.
2. Alternatively, you may wish to use `AutoRTFM::UnreachableIfClosed()` or
   `AutoRTFM::UnreachableIfTransactional()` in the caller function, before the
   call to the disabled callee. This will silence the compiler error, but if
   the caller function is itself called by a closed transaction, then the
   runtime will assert and crash.\
   This can be used as a short-term solution until `AUTORTFM_DISABLE` can be
   correctly applied.
3. If the callee function is a templated function or class that calls a
   template-dependent function, it may be best to apply `AUTORTFM_INFER` to the
   callee function. This will act similarly to
   [AUTORTFM_DISABLE](#autortfm-disabled) if and only if the templated function
   calls an AutoRTFM-disabled function.

If you're an Epic employee and need assistance, please don't hesitate to request help in
[#verse-autortfm-dev-ext](https://epic.enterprise.slack.com/archives/C059TRJ6XJ4)

### Is there an easy way to visualize the AutoRTFM mode of a function / class?

Yes!\
There is an AutoRTFM-enhanced version of `clangd` at
`Restricted/NotForLicensees/Binaries/Win64/AutoRTFM/20/bin/verse-clangd.exe`
which understands the AutoRTFM attributes and displays AutoRTFM modes via hover
info and inlay-hints.

`verse-clangd` can be used as a drop in replacement of `clangd` with your favorite IDE.
