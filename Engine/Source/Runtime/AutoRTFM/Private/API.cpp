// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/CAPI.h"
#include "AutoRTFM/Defines.h"
#include "Context.h"
#include "ExternAPI.h"
#include "Toggles.h"
#include "Transaction.h"

#include <random>
#include <tuple>
#include <utility>

#if UE_AUTORTFM
static_assert(UE_AUTORTFM_ENABLED, "AutoRTFM/API.cpp requires the compiler flag '-fautortfm'");

namespace
{
// Move this to a local only and use functions to access this
#if UE_AUTORTFM_ENABLED_RUNTIME_BY_DEFAULT
int GAutoRTFMRuntimeEnabled = AutoRTFM::ForTheRuntime::EAutoRTFMEnabledState::AutoRTFM_EnabledByDefault;
#else
int GAutoRTFMRuntimeEnabled = AutoRTFM::ForTheRuntime::EAutoRTFMEnabledState::AutoRTFM_DisabledByDefault;
#endif  // UE_AUTORTFM_ENABLED_RUNTIME_BY_DEFAULT

int GAutoRTFMInternalAbortAction = AutoRTFM::ForTheRuntime::EAutoRTFMInternalAbortActionState::Crash;
int GAutoRTFMRetryTransactions = AutoRTFM::ForTheRuntime::EAutoRTFMRetryTransactionState::NoRetry;

// Set the percentage chance [0..100] that AutoRTFM will be enabled.
// 100.0 means that AutoRTFM will always be enabled; 1.0 means that AutoRTFM has a 1% chance of being enabled.
// See `AutoRTFM::ForTheRuntime::CoinTossDisable` for implementation details.
float GAutoRTFMEnabledProbability = 100.0f;

bool GAutoRTFMEnsureOnInternalAbort = true;

// A linked-list of open->closed function tables populated by
// autortfm_register_open_to_closed_functions(). This is consumed by
// ProcessAllPendingOpenToClosedRegistrations() when autortfm_initialize()
// is called.
autortfm_open_to_closed_table* GPendingOpenToClosedRegistrations = nullptr;
}

constexpr int GMaxRegisteredHeapRedirects = 4;
AutoRTFM::FHeapRedirectCallbacks GRegisteredHeapRedirects[GMaxRegisteredHeapRedirects];
int GNumRegisteredHeapRedirects = 0;

AutoRTFM::FHeapRedirectCallbacks* FindHeapRedirectCallbacksForAddressSpace(uint32_t AddressSpace)
{
	static AutoRTFM::FHeapRedirectCallbacks* CachedRedirectCallbacks = nullptr;

	if (CachedRedirectCallbacks && (CachedRedirectCallbacks->AddressSpace == AddressSpace))
	{
		return CachedRedirectCallbacks;
	}

	for (int RedirectedHeapIndex = 0; RedirectedHeapIndex < GNumRegisteredHeapRedirects; RedirectedHeapIndex++)
	{
		AutoRTFM::FHeapRedirectCallbacks* HeapRedirects = &GRegisteredHeapRedirects[RedirectedHeapIndex];

		if (HeapRedirects->AddressSpace == AddressSpace)
		{
			CachedRedirectCallbacks = HeapRedirects;
			return HeapRedirects;
		}
	}

	return nullptr;
}
#endif  // UE_AUTORTFM

namespace AutoRTFM
{
namespace Testing
{
UE_AUTORTFM_API ForTheRuntime::EAutoRTFMEnabledState ForceSetAutoRTFMRuntime(ForTheRuntime::EAutoRTFMEnabledState State)
{
#if UE_AUTORTFM
	const int Original = GAutoRTFMRuntimeEnabled;
	if (GAutoRTFMRuntimeEnabled != State)
	{
		GAutoRTFMRuntimeEnabled = State;
		if (GExternAPI.OnRuntimeEnabledChanged)
		{
			GExternAPI.OnRuntimeEnabledChanged();
		}
	}
	return static_cast<ForTheRuntime::EAutoRTFMEnabledState>(Original);
#else
	UE_AUTORTFM_UNUSED(State);
	return ForTheRuntime::AutoRTFM_ForcedDisabled;
#endif
}
}  // namespace Testing
}  // namespace AutoRTFM

#if UE_AUTORTFM
namespace AutoRTFM
{
AUTORTFM_INTERNAL bool IsAutoRTFMInitialized()
{
	return FContext::Get() != nullptr;
}
}
#endif

namespace AutoRTFM
{

namespace ForTheRuntime
{
bool SetAutoRTFMRuntime(EAutoRTFMEnabledState State)
{
	// #noop if AutoRTFM is not compiled in
#if UE_AUTORTFM
	auto Stringify = [](int Query) -> const char*
	{
		switch (Query)
		{
			default:
				InternalUnreachable();
#define HANDLE_CASE(x) \
	case x:            \
		return #x
				HANDLE_CASE(EAutoRTFMEnabledState::AutoRTFM_ForcedEnabled);
				HANDLE_CASE(EAutoRTFMEnabledState::AutoRTFM_ForcedDisabled);
				HANDLE_CASE(EAutoRTFMEnabledState::AutoRTFM_OverriddenEnabled);
				HANDLE_CASE(EAutoRTFMEnabledState::AutoRTFM_OverriddenDisabled);
				HANDLE_CASE(EAutoRTFMEnabledState::AutoRTFM_Enabled);
				HANDLE_CASE(EAutoRTFMEnabledState::AutoRTFM_Disabled);
				HANDLE_CASE(EAutoRTFMEnabledState::AutoRTFM_EnabledByDefault);
				HANDLE_CASE(EAutoRTFMEnabledState::AutoRTFM_DisabledByDefault);
#undef HANDLE_CASE
		}
	};

	auto DoIgnoreLog = [&](int State, int Stored)
	{
		AUTORTFM_LOG("Ignoring changing AutoRTFM runtime state to '%s' as it was previously set to '%s'", Stringify(State),
			Stringify(Stored));
	};

	switch (GAutoRTFMRuntimeEnabled)
	{
		default:
			break;
		case EAutoRTFMEnabledState::AutoRTFM_ForcedEnabled:
		case EAutoRTFMEnabledState::AutoRTFM_ForcedDisabled:
			DoIgnoreLog(State, GAutoRTFMRuntimeEnabled);
			return false;
	}

	switch (GAutoRTFMRuntimeEnabled)
	{
		default:
			break;
		case EAutoRTFMEnabledState::AutoRTFM_OverriddenEnabled:
		case EAutoRTFMEnabledState::AutoRTFM_OverriddenDisabled:
			if ((State == EAutoRTFMEnabledState::AutoRTFM_Enabled) || (State == EAutoRTFMEnabledState::AutoRTFM_Disabled)
				|| (State == EAutoRTFMEnabledState::AutoRTFM_EnabledByDefault)
				|| (State == EAutoRTFMEnabledState::AutoRTFM_DisabledByDefault))
			{
				DoIgnoreLog(State, GAutoRTFMRuntimeEnabled);
				return false;
			}
		case EAutoRTFMEnabledState::AutoRTFM_Enabled:
		case EAutoRTFMEnabledState::AutoRTFM_Disabled:
			if ((State == EAutoRTFMEnabledState::AutoRTFM_EnabledByDefault)
				|| (State == EAutoRTFMEnabledState::AutoRTFM_DisabledByDefault))
			{
				DoIgnoreLog(State, GAutoRTFMRuntimeEnabled);
				return false;
			}
	}

	if (GAutoRTFMRuntimeEnabled != State)
	{
		GAutoRTFMRuntimeEnabled = State;
		if (GExternAPI.OnRuntimeEnabledChanged)
		{
			GExternAPI.OnRuntimeEnabledChanged();
		}
	}

	return true;
#else
	UE_AUTORTFM_UNUSED(State);
	return false;
#endif
}

bool IsAutoRTFMRuntimeEnabledInternal()
{
	// #noop if AutoRTFM is not compiled in
#if UE_AUTORTFM
	switch (GAutoRTFMRuntimeEnabled)
	{
		default:
			return false;
		case EAutoRTFMEnabledState::AutoRTFM_Enabled:
		case EAutoRTFMEnabledState::AutoRTFM_ForcedEnabled:
		case EAutoRTFMEnabledState::AutoRTFM_OverriddenEnabled:
		case EAutoRTFMEnabledState::AutoRTFM_EnabledByDefault:
			return true;
	}
#else
	return false;
#endif
}

void SetAutoRTFMEnabledProbability(float Chance)
{
#if UE_AUTORTFM
	AUTORTFM_ASSERT(Chance >= 0.0f && Chance <= 100.0f);
	GAutoRTFMEnabledProbability = Chance;
#else
	UE_AUTORTFM_UNUSED(Chance);
#endif
}

float GetAutoRTFMEnabledProbability()
{
#if UE_AUTORTFM
	return GAutoRTFMEnabledProbability;
#else
	return 0.0f;
#endif
}

bool CoinTossDisable()
{
#if UE_AUTORTFM
	if (!IsAutoRTFMRuntimeEnabled())
	{
		return false;
	}

	static std::random_device Device;
	static std::mt19937 Generator(Device());
	static std::uniform_real_distribution<float> Distribution(0.0f, 100.0f);

	// A value in the range [0..100), EG. inclusive of 0, exclusive of 100.
	// So a `GAutoRTFMEnabledProbability` of 100 is always greater than the
	// potential random range, and `GAutoRTFMEnabledProbability` of 0 is
	// always less than or equal to the range.
	const float Random = Distribution(Generator);

	if (GAutoRTFMEnabledProbability <= Random)
	{
		// If we have the runtime cvar set to `ForcedEnabled` then it'll ignore this call!
		return AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::AutoRTFM_ForcedDisabled);
	}
#endif

	return false;
}

void SetInternalAbortAction(EAutoRTFMInternalAbortActionState State)
{
#if UE_AUTORTFM
	GAutoRTFMInternalAbortAction = State;
#else
	UE_AUTORTFM_UNUSED(State);
#endif
}

EAutoRTFMInternalAbortActionState GetInternalAbortAction()
{
#if UE_AUTORTFM
	return static_cast<EAutoRTFMInternalAbortActionState>(GAutoRTFMInternalAbortAction);
#else
	return Crash;
#endif
}

bool GetEnsureOnInternalAbort()
{
#if UE_AUTORTFM
	return GAutoRTFMEnsureOnInternalAbort;
#else
	return false;
#endif
}

void SetEnsureOnInternalAbort([[maybe_unused]] bool bEnabled)
{
#if UE_AUTORTFM
	GAutoRTFMEnsureOnInternalAbort = bEnabled;
#endif
}

void SetRetryTransaction(EAutoRTFMRetryTransactionState State)
{
#if UE_AUTORTFM
	if (GAutoRTFMRetryTransactions != State)
	{
		GAutoRTFMRetryTransactions = State;
		if (GExternAPI.OnRetryTransactionsChanged)
		{
			GExternAPI.OnRetryTransactionsChanged();
		}
	}
#else
	UE_AUTORTFM_UNUSED(State);
#endif
}

EAutoRTFMRetryTransactionState GetRetryTransaction()
{
#if UE_AUTORTFM
	return static_cast<EAutoRTFMRetryTransactionState>(GAutoRTFMRetryTransactions);
#else
	return NoRetry;
#endif
}

bool ShouldRetryNonNestedTransactions()
{
#if UE_AUTORTFM
	switch (GAutoRTFMRetryTransactions)
	{
		default:
			return false;
		case EAutoRTFMRetryTransactionState::RetryNonNested:
		case EAutoRTFMRetryTransactionState::RetryNestedToo:
			return true;
	}
#else
	return false;
#endif
}

bool ShouldRetryNestedTransactionsToo()
{
#if UE_AUTORTFM
	switch (GAutoRTFMRetryTransactions)
	{
		default:
			return false;
		case EAutoRTFMRetryTransactionState::RetryNestedToo:
			return true;
	}
#else
	return false;
#endif
}
}  // namespace ForTheRuntime
}  // namespace AutoRTFM

#if (defined(__AUTORTFM_ENABLED) && __AUTORTFM_ENABLED)
#include "AutoRTFM/Constants.h"
#include "CallNest.h"
#include "Context.h"
#include "ContextInlines.h"
#include "FunctionMapInlines.h"
#include "Toggles.h"
#include "TransactionInlines.h"
#include "Utils.h"

// This is the implementation of the AutoRTFM.h API. Ideally, functions here should just delegate to some internal API.
// For now, I have these functions also perform some error checking.

namespace AutoRTFM
{

namespace
{

// Internal closed-variant implementations.
AUTORTFM_INTERNAL autortfm_transaction_status RTFM_autortfm_transact(
	void (*UninstrumentedWork)(void*), void (*InstrumentedWork)(void*), void* Arg)
{
	return static_cast<autortfm_transaction_status>(FContext::Get()->Transact(UninstrumentedWork, InstrumentedWork, Arg));
}

UE_AUTORTFM_FORCEINLINE autortfm_transaction_status TransactThenOpenImpl(void (*UninstrumentedWork)(void*), void* Arg)
{
	return static_cast<autortfm_transaction_status>(AutoRTFM::Transact([&] { autortfm_open(UninstrumentedWork, Arg); }));
}

AUTORTFM_INTERNAL autortfm_transaction_status RTFM_autortfm_transact_then_open(void (*UninstrumentedWork)(void*), void* Arg)
{
	return TransactThenOpenImpl(UninstrumentedWork, Arg);
}

AUTORTFM_INTERNAL autortfm_transaction_status RTFM_autortfm_close(
	void (*UninstrumentedWork)(void*), void (*InstrumentedWork)(void*), void* Arg)
{
	if (AUTORTFM_LIKELY(InstrumentedWork))
	{
		InstrumentedWork(Arg);
	}
	else
	{
		AUTORTFM_REPORT_ERROR("Could not find function %p '%s' for autortfm_close()", reinterpret_cast<void*>(UninstrumentedWork),
			GetFunctionDescription(UninstrumentedWork).c_str());
	}

	return autortfm_transaction_executing;
}

#if AUTORTFM_SANITIZER
extern "C" UE_AUTORTFM_NOAUTORTFM UE_AUTORTFM_API Sanitizer::FDisableHandle autortfm_disable_sanitizer() noexcept
{
	return Sanitizer::BeginDisable();
}

extern "C" UE_AUTORTFM_NOAUTORTFM UE_AUTORTFM_API void autortfm_enable_sanitizer(Sanitizer::FDisableHandle Handle) noexcept
{
	Sanitizer::EndDisable(Handle);
}
#endif  // AUTORTFM_SANITIZER

extern "C" UE_AUTORTFM_NOAUTORTFM UE_AUTORTFM_API void autortfm_maybe_throw() noexcept
{
	FContext* const Context = FContext::Get();
	Context->MaybeThrow();
}

extern "C" UE_AUTORTFM_NOAUTORTFM UE_AUTORTFM_API void autortfm_pre_static_local_initializer() noexcept
{
	if (FContext* Context = FContext::Get())
	{
		Context->EnteringStaticLocalInitializer();
	}
}

extern "C" UE_AUTORTFM_NOAUTORTFM UE_AUTORTFM_API void autortfm_post_static_local_initializer() noexcept
{
	if (FContext* Context = FContext::Get())
	{
		Context->LeavingStaticLocalInitializer();
	}
}

AUTORTFM_INTERNAL void RTFM_autortfm_record_open_write_err(void*, size_t)
{
	AUTORTFM_FATAL("The function `autortfm_record_open_write` was called from closed code");
}

AUTORTFM_INTERNAL void RTFM_OnCommitInternal(TTask<void()>&& Work)
{
	FContext* const Context = FContext::Get();
	AUTORTFM_ASSERT(Context->IsTransactional());
	Context->GetCurrentTransaction()->DeferUntilCommit(std::move(Work));
}

AUTORTFM_INTERNAL void RTFM_OnPreAbortInternal(TTask<void()>&& Work)
{
	FContext* const Context = FContext::Get();
	AUTORTFM_ASSERT(Context->IsTransactional());
	Context->GetCurrentTransaction()->DeferUntilPreAbort(std::move(Work));
}

AUTORTFM_INTERNAL void RTFM_OnAbortInternal(TTask<void()>&& Work)
{
	FContext* const Context = FContext::Get();
	AUTORTFM_ASSERT(Context->IsTransactional());
	Context->GetCurrentTransaction()->DeferUntilAbort(std::move(Work));
}

AUTORTFM_INTERNAL void RTFM_OnCompleteInternal(TTask<void()>&& Work)
{
	FContext* const Context = FContext::Get();
	AUTORTFM_ASSERT(Context->IsTransactional());
	Context->DeferUntilComplete(std::move(Work));
}

AUTORTFM_INTERNAL void RTFM_OnRetryInternal(TTask<void()>&& Work)
{
	FContext* const Context = FContext::Get();
	AUTORTFM_ASSERT(Context->IsTransactional());
	Context->DeferUntilRetry(std::move(Work));
}

AUTORTFM_INTERNAL void RTFM_PushOnCommitHandlerInternal(const void* Key, TTask<void()>&& Work)
{
	FContext* const Context = FContext::Get();
	AUTORTFM_ASSERT(Context->IsTransactional());
	Context->GetCurrentTransaction()->PushDeferUntilCommitHandler(Key, std::move(Work));
}

AUTORTFM_INTERNAL void RTFM_PopOnCommitHandlerInternal(const void* Key)
{
	FContext* const Context = FContext::Get();
	AUTORTFM_ASSERT(Context->IsTransactional());
	Context->GetCurrentTransaction()->PopDeferUntilCommitHandler(Key);
}

AUTORTFM_INTERNAL void RTFM_PopAllOnCommitHandlersInternal(const void* Key)
{
	FContext* const Context = FContext::Get();
	AUTORTFM_ASSERT(Context->IsTransactional());
	Context->GetCurrentTransaction()->PopAllDeferUntilCommitHandlers(Key);
}

AUTORTFM_INTERNAL void RTFM_PushOnAbortHandlerInternal(const void* Key, TTask<void()>&& Work)
{
	FContext* const Context = FContext::Get();
	AUTORTFM_ASSERT(Context->IsTransactional());
	Context->GetCurrentTransaction()->PushDeferUntilAbortHandler(Key, std::move(Work));
}

AUTORTFM_INTERNAL void RTFM_PopOnAbortHandlerInternal(const void* Key)
{
	FContext* const Context = FContext::Get();
	AUTORTFM_ASSERT(Context->IsTransactional());
	Context->GetCurrentTransaction()->PopDeferUntilAbortHandler(Key);
}

AUTORTFM_INTERNAL void RTFM_PopAllOnAbortHandlersInternal(const void* Key)
{
	FContext* const Context = FContext::Get();
	AUTORTFM_ASSERT(Context->IsTransactional());
	Context->GetCurrentTransaction()->PopAllDeferUntilAbortHandlers(Key);
}

AUTORTFM_INTERNAL void RTFM_autortfm_on_commit(void (*Work)(void*), void* Arg)
{
	RTFM_OnCommitInternal([Work, Arg] { Work(Arg); });
}

AUTORTFM_INTERNAL void RTFM_autortfm_on_pre_abort(void (*Work)(void*), void* Arg)
{
	RTFM_OnPreAbortInternal([Work, Arg] { Work(Arg); });
}

AUTORTFM_INTERNAL void RTFM_autortfm_on_abort(void (*Work)(void*), void* Arg)
{
	RTFM_OnAbortInternal([Work, Arg] { Work(Arg); });
}

AUTORTFM_INTERNAL void RTFM_autortfm_push_on_abort_handler(const void* Key, void (*Work)(void*), void* Arg)
{
	RTFM_PushOnAbortHandlerInternal(Key, [Work, Arg] { Work(Arg); });
}

AUTORTFM_INTERNAL void RTFM_autortfm_pop_on_abort_handler(const void* Key)
{
	RTFM_PopOnAbortHandlerInternal(Key);
}

AUTORTFM_INTERNAL void* RTFM_autortfm_did_allocate(void* Ptr, size_t Size)
{
	FContext* const Context = FContext::Get();
	Context->DidAllocate(Ptr, Size);
	return Ptr;
}

AUTORTFM_INTERNAL void RTFM_autortfm_did_free(void* Ptr)
{
	// We should never-ever-ever actually free memory from within closed code of
	// a transaction.
	AutoRTFM::InternalUnreachable();
}

// Consume the GPendingOpenToClosedRegistrations linked list to register the
// open -> closed functions. This is done via a linked-list to avoid heap
// allocations before AutoRTFM is initialized.
AUTORTFM_INTERNAL void ProcessAllPendingOpenToClosedRegistrations()
{
	AUTORTFM_ASSERT(IsAutoRTFMInitialized());
	FunctionMapAdd(GPendingOpenToClosedRegistrations);
	GPendingOpenToClosedRegistrations = nullptr;
}

}  // anonymous namespace

// Populated by the AutoRTFM compiler
extern "C" autortfm_open_to_closed_table autortfm_external_mappings;

// The AutoRTFM public API.

extern "C" void autortfm_initialize(const autortfm_extern_api* ExternAPI) noexcept
{
	AutoRTFM::UnreachableIfClosed("TODO: Mark this as AUTORTFM_DISABLED and fixup callers");
	AUTORTFM_ENSURE_MSG(!FContext::Get(), "AutoRTFM initialized twice");

	autortfm_register_open_to_closed_functions(&autortfm_external_mappings);

	AUTORTFM_ASSERT(ExternAPI);
	AUTORTFM_ASSERT(ExternAPI->Allocate);
	AUTORTFM_ASSERT(ExternAPI->Reallocate);
	AUTORTFM_ASSERT(ExternAPI->AllocateZeroed);
	AUTORTFM_ASSERT(ExternAPI->Free);
	AUTORTFM_ASSERT(ExternAPI->Log);
	AUTORTFM_ASSERT(ExternAPI->LogWithCallstack);
	AUTORTFM_ASSERT(ExternAPI->EnsureFailure);
	AUTORTFM_ASSERT(ExternAPI->IsLogActive);
	AUTORTFM_ASSERT(ExternAPI->CaptureCallstack);
	AUTORTFM_ASSERT(ExternAPI->LogCallstack);

	GExternAPI = *ExternAPI;
	FContext::Create();
	ProcessAllPendingOpenToClosedRegistrations();
	AUTORTFM_SANITIZER_ONLY(Sanitizer::Initialize());
}

extern "C" void autortfm_shutdown() noexcept
{
	AutoRTFM::UnreachableIfClosed("TODO: Mark this as AUTORTFM_DISABLED and fixup callers");

	if (!FContext::Get())
	{
		// autortfm_shutdown() was likely called before autortfm_initialize().
		// This is a violation of the API, but can happen if the process errors
		// on initialization. Just return, so the process can exit cleanly.
		return;
	}

	FContext::Destroy();
	AUTORTFM_SANITIZER_ONLY(Sanitizer::Shutdown());
}

extern "C" UE_AUTORTFM_ALWAYS_OPEN autortfm_context_status autortfm_get_context_status() noexcept
{
	if (ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		if (FContext* Context = FContext::Get())
		{
			return static_cast<autortfm_context_status>(Context->GetStatus());
		}
	}

	return autortfm_status_idle;
}

extern "C" UE_AUTORTFM_ALWAYS_OPEN bool autortfm_is_context_status(autortfm_context_status Status) noexcept
{
	return autortfm_get_context_status() == Status;
}

extern "C" autortfm_transaction_status autortfm_transact(
	void (*UninstrumentedWork)(void*), void (*InstrumentedWork)(void*), void* Arg)
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_transact(UninstrumentedWork, InstrumentedWork, Arg);
	}

	if (ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		return static_cast<autortfm_transaction_status>(FContext::Get()->Transact(UninstrumentedWork, InstrumentedWork, Arg));
	}

	(*UninstrumentedWork)(Arg);
	return autortfm_transaction_committed;
}

extern "C" autortfm_transaction_status autortfm_transact_then_open(void (*UninstrumentedWork)(void*), void* Arg)
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_transact_then_open(UninstrumentedWork, Arg);
	}

	return TransactThenOpenImpl(UninstrumentedWork, Arg);
}

extern "C" autortfm_transaction_id autortfm_current_transaction_id() noexcept
{
	if (FContext* const Context = FContext::Get())
	{
		if (FTransaction* Transaction = Context->GetCurrentTransaction())
		{
			return static_cast<autortfm_transaction_id>(Transaction->Identifier());
		}
	}
	return 0;
}

// #jira SOL-9048: in theory we should be able to go back to using the .aem file for this.
AUTORTFM_INTERNAL static void autortfm_abort_transaction_from_closed() noexcept
{
	FContext::Get()->AbortTransactionAndThrow(ETransactionStatus::AbortedByRequest);
}

extern "C" void autortfm_abort_transaction() noexcept
{
	if (autortfm_is_closed())
	{
		autortfm_abort_transaction_from_closed();
	}
	else
	{
		FContext::Get()->AbortTransaction(ETransactionStatus::AbortedByRequest);
	}
}

// #jira SOL-9048: in theory we should be able to go back to using the .aem file for this.
AUTORTFM_INTERNAL static void autortfm_cascading_abort_transaction_from_closed() noexcept
{
	FContext::Get()->AbortTransactionAndThrow(ETransactionStatus::AbortedByCascadingAbort);
}

extern "C" void autortfm_cascading_abort_transaction() noexcept
{
	if (autortfm_is_closed())
	{
		autortfm_cascading_abort_transaction_from_closed();
	}
	else
	{
		FContext::Get()->AbortTransaction(ETransactionStatus::AbortedByCascadingAbort);
	}
}

// #jira SOL-9048: in theory we should be able to go back to using the .aem file for this.
AUTORTFM_INTERNAL static void autortfm_cascading_retry_transaction_from_closed() noexcept
{
	FContext::Get()->AbortTransactionAndThrow(ETransactionStatus::AbortedByCascadingRetry);
}

extern "C" void autortfm_cascading_retry_transaction() noexcept
{
	if (autortfm_is_closed())
	{
		autortfm_cascading_retry_transaction_from_closed();
	}
	else
	{
		FContext::Get()->AbortTransaction(ETransactionStatus::AbortedByCascadingRetry);
	}
}

extern "C" void autortfm_start_transaction() noexcept
{
	FContext::Get()->StartTransaction();
}

extern "C" void autortfm_commit_transaction() noexcept
{
	FContext::Get()->CommitTransaction();
}

// #jira SOL-9048: in theory we should be able to go back to using the .aem file for this.
AUTORTFM_INTERNAL static void autortfm_open_from_closed(void (*Work)(void*), void* Arg)
{
	FContext* const Context = FContext::Get();

	Work(Arg);

	Context->MaybeThrow();
}

extern "C" void autortfm_open(void (*Work)(void*), void* Arg)
{
	if (autortfm_is_closed())
	{
		autortfm_open_from_closed(Work, Arg);
	}
	else
	{
		Work(Arg);
	}
}

// #jira SOL-9048: in theory we should be able to go back to using the .aem file for this.
AUTORTFM_INTERNAL static void autortfm_open_from_closed_no_sanitize(void (*Work)(void*), void* Arg)
{
	FContext* const Context = FContext::Get();

	AUTORTFM_SANITIZER_ONLY(Sanitizer::FDisableHandle DisableHandle = Sanitizer::BeginDisable());

	Work(Arg);

	AUTORTFM_SANITIZER_ONLY(Sanitizer::EndDisable(DisableHandle));

	Context->MaybeThrow();
}

extern "C" void autortfm_open_no_sanitize(void (*Work)(void*), void* Arg)
{
	if (autortfm_is_closed())
	{
		autortfm_open_from_closed_no_sanitize(Work, Arg);
	}
	else
	{
		Work(Arg);
	}
}

extern "C" autortfm_transaction_status autortfm_close(void (*UninstrumentedWork)(void*), void (*InstrumentedWork)(void*), void* Arg)
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_close(UninstrumentedWork, InstrumentedWork, Arg);
	}

	autortfm_transaction_status Result = autortfm_transaction_executing;

	if (ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		AUTORTFM_FATAL_IF(!FContext::Get()->IsTransactional(), "Close called from an outside a transaction");

		FContext* const Context = FContext::Get();

		if (AUTORTFM_LIKELY(InstrumentedWork))
		{
			Result = static_cast<autortfm_transaction_status>(Context->CallClosedNest(InstrumentedWork, Arg));
		}
		else
		{
			std::string FunctionDescription = GetFunctionDescription(UninstrumentedWork);
			if (ForTheRuntime::GetInternalAbortAction() == ForTheRuntime::EAutoRTFMInternalAbortActionState::Crash)
			{
				AUTORTFM_FATAL(
					"Could not find function %p '%s' in autortfm_close()", UninstrumentedWork, FunctionDescription.c_str());
			}
			else
			{
				AUTORTFM_ENSURE_MSG(!ForTheRuntime::GetEnsureOnInternalAbort(),
					"Could not find function %p '%s' in autortfm_close()", UninstrumentedWork, FunctionDescription.c_str());
			}
			Context->AbortTransactionAndThrow(ETransactionStatus::AbortedByLanguage);
		}
	}
	else
	{
		UninstrumentedWork(Arg);
	}

	return Result;
}

extern "C" void autortfm_record_open_write(void* Ptr, size_t Size) noexcept
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_record_open_write_err(Ptr, Size);
	}
	else if (FContext* const Context = FContext::Get(); Context && Context->IsTransactional())
	{
		if (FTransaction* CurrentTransaction = Context->GetCurrentTransaction())
		{
			CurrentTransaction->RecordWrite(Ptr, Size);
		}
	}
}

extern "C" void autortfm_record_open_write_with_flags(void* Ptr, size_t Size, autortfm_write_flags Flags) noexcept
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_record_open_write_err(Ptr, Size);
	}
	else if (FContext* const Context = FContext::Get(); Context && Context->IsTransactional())
	{
		if (FTransaction* CurrentTransaction = Context->GetCurrentTransaction())
		{
			CurrentTransaction->RecordWrite(Ptr, Size, static_cast<EWriteFlags>(Flags));
		}
	}
}

extern "C" UE_AUTORTFM_NOAUTORTFM void autortfm_register_open_to_closed_functions(autortfm_open_to_closed_table* Table) noexcept
{
	Table->Next = GPendingOpenToClosedRegistrations;
	GPendingOpenToClosedRegistrations = Table;

	if (IsAutoRTFMInitialized())
	{
		ProcessAllPendingOpenToClosedRegistrations();
	}
}

extern "C" UE_AUTORTFM_NOAUTORTFM void autortfm_unregister_open_to_closed_functions(autortfm_open_to_closed_table* Table) noexcept
{
	if (Table == GPendingOpenToClosedRegistrations)
	{
		GPendingOpenToClosedRegistrations = Table->Next;
	}
	if (Table->Next)
	{
		Table->Next->Prev = Table->Prev;
	}
	if (Table->Prev)
	{
		Table->Prev->Next = Table->Next;
	}
	Table->Prev = nullptr;
	Table->Next = nullptr;

	// Note: If AutoRTFM is already initialized, we currently do *not* remove
	// the registered functions from the function map. The reason for this is
	// that we can register the same open address multiple times, where the
	// closed address uses the value of the last register call.
	// To support unregistering these cleanly, we'd need to increase the
	// complexity of the function map - either by storing a list of all the
	// closed functions that were registered for an open, or entirely rebuilding
	// the map from the autortfm_open_to_closed_table lists. So far, keeping
	// stale mappings has not been an issue, but if it does become an issue,
	// then something will need to be done here.
}

extern "C" UE_AUTORTFM_ALWAYS_OPEN bool autortfm_is_on_current_transaction_stack(void* Ptr) noexcept
{
	if (FContext* Context = FContext::Get())
	{
		if (!Context->IsTransactional())
		{
			return false;
		}
		if (FTransaction* CurrentTransaction = Context->GetCurrentTransaction())
		{
			return CurrentTransaction->IsOnStack(Ptr);
		}
	}
	return false;
}

void ForTheRuntime::OnCommitInternal(TTask<void()>&& Work)
{
	if (autortfm_is_closed())
	{
		return RTFM_OnCommitInternal(std::move(Work));
	}

	Work();
}

void ForTheRuntime::OnPreAbortInternal(TTask<void()>&& Work)
{
	if (autortfm_is_closed())
	{
		return RTFM_OnPreAbortInternal(std::move(Work));
	}
}
void ForTheRuntime::OnAbortInternal(TTask<void()>&& Work)
{
	if (autortfm_is_closed())
	{
		return RTFM_OnAbortInternal(std::move(Work));
	}
}

void ForTheRuntime::OnCompleteInternal(TTask<void()>&& Work)
{
	if (autortfm_is_closed())
	{
		return RTFM_OnCompleteInternal(std::move(Work));
	}
}

void ForTheRuntime::OnRetryInternal(TTask<void()>&& Work)
{
	if (autortfm_is_closed())
	{
		return RTFM_OnRetryInternal(std::move(Work));
	}
}

void ForTheRuntime::PushOnCommitHandlerInternal(const void* Key, TTask<void()>&& Work)
{
	if (autortfm_is_closed())
	{
		return RTFM_PushOnCommitHandlerInternal(Key, std::move(Work));
	}
}

void ForTheRuntime::PopOnCommitHandlerInternal(const void* Key)
{
	if (autortfm_is_closed())
	{
		return RTFM_PopOnCommitHandlerInternal(Key);
	}
}

void ForTheRuntime::PopAllOnCommitHandlersInternal(const void* Key)
{
	if (autortfm_is_closed())
	{
		return RTFM_PopAllOnCommitHandlersInternal(Key);
	}
}

void ForTheRuntime::PushOnAbortHandlerInternal(const void* Key, TTask<void()>&& Work)
{
	if (autortfm_is_closed())
	{
		return RTFM_PushOnAbortHandlerInternal(Key, std::move(Work));
	}
}

void ForTheRuntime::PopOnAbortHandlerInternal(const void* Key)
{
	if (autortfm_is_closed())
	{
		return RTFM_PopOnAbortHandlerInternal(Key);
	}
}

void ForTheRuntime::PopAllOnAbortHandlersInternal(const void* Key)
{
	if (autortfm_is_closed())
	{
		return RTFM_PopAllOnAbortHandlersInternal(Key);
	}
}

UE_AUTORTFM_NOAUTORTFM
void ForTheRuntime::RegisterOnCommitFromTheOpen(TTask<void()>&& Work)
{
	RTFM_OnCommitInternal(std::move(Work));
}

UE_AUTORTFM_NOAUTORTFM
void ForTheRuntime::RegisterOnAbortFromTheOpen(TTask<void()>&& Work)
{
	RTFM_OnAbortInternal(std::move(Work));
}

void ForTheRuntime::RedirectedLoad(
	uint32_t AddressSpace, void* DestPointer, uint64_t Size, uint64_t SourceAddress, bool bWillWriteHint)
{
	if (FHeapRedirectCallbacks* HeapRedirects = FindHeapRedirectCallbacksForAddressSpace(AddressSpace))
	{
		HeapRedirects->RedirectedLoad(DestPointer, Size, SourceAddress, bWillWriteHint);
	}
}

void ForTheRuntime::RedirectedLoad8(uint32_t AddressSpace, void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint)
{
	if (FHeapRedirectCallbacks* HeapRedirects = FindHeapRedirectCallbacksForAddressSpace(AddressSpace))
	{
		HeapRedirects->RedirectedLoad8(DestPointer, SourceAddress, bWillWriteHint);
	}
}

void ForTheRuntime::RedirectedLoad4(uint32_t AddressSpace, void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint)
{
	if (FHeapRedirectCallbacks* HeapRedirects = FindHeapRedirectCallbacksForAddressSpace(AddressSpace))
	{
		HeapRedirects->RedirectedLoad4(DestPointer, SourceAddress, bWillWriteHint);
	}
}

void ForTheRuntime::RedirectedLoad2(uint32_t AddressSpace, void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint)
{
	if (FHeapRedirectCallbacks* HeapRedirects = FindHeapRedirectCallbacksForAddressSpace(AddressSpace))
	{
		HeapRedirects->RedirectedLoad2(DestPointer, SourceAddress, bWillWriteHint);
	}
}

void ForTheRuntime::RedirectedLoad1(uint32_t AddressSpace, void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint)
{
	if (FHeapRedirectCallbacks* HeapRedirects = FindHeapRedirectCallbacksForAddressSpace(AddressSpace))
	{
		HeapRedirects->RedirectedLoad1(DestPointer, SourceAddress, bWillWriteHint);
	}
}

void ForTheRuntime::RedirectedStore(uint32_t AddressSpace, uint64_t DestAddress, uint64_t Size, const void* SourcePointer)
{
	if (FHeapRedirectCallbacks* HeapRedirects = FindHeapRedirectCallbacksForAddressSpace(AddressSpace))
	{
		HeapRedirects->RedirectedStore(DestAddress, Size, SourcePointer);
	}
}

void ForTheRuntime::RedirectedStore8(uint32_t AddressSpace, uint64_t DestAddress, const void* SourcePointer)
{
	if (FHeapRedirectCallbacks* HeapRedirects = FindHeapRedirectCallbacksForAddressSpace(AddressSpace))
	{
		HeapRedirects->RedirectedStore8(DestAddress, SourcePointer);
	}
}

void ForTheRuntime::RedirectedStore4(uint32_t AddressSpace, uint64_t DestAddress, const void* SourcePointer)
{
	if (FHeapRedirectCallbacks* HeapRedirects = FindHeapRedirectCallbacksForAddressSpace(AddressSpace))
	{
		HeapRedirects->RedirectedStore4(DestAddress, SourcePointer);
	}
}

void ForTheRuntime::RedirectedStore2(uint32_t AddressSpace, uint64_t DestAddress, const void* SourcePointer)
{
	if (FHeapRedirectCallbacks* HeapRedirects = FindHeapRedirectCallbacksForAddressSpace(AddressSpace))
	{
		HeapRedirects->RedirectedStore2(DestAddress, SourcePointer);
	}
}

void ForTheRuntime::RedirectedStore1(uint32_t AddressSpace, uint64_t DestAddress, const void* SourcePointer)
{
	if (FHeapRedirectCallbacks* HeapRedirects = FindHeapRedirectCallbacksForAddressSpace(AddressSpace))
	{
		HeapRedirects->RedirectedStore1(DestAddress, SourcePointer);
	}
}

void Initialize(const FExternAPI& ExternAPI)
{
	autortfm_initialize(&ExternAPI);
}

void Shutdown()
{
	autortfm_shutdown();
}

void RegisterHeapRedirectCallbacks(FHeapRedirectCallbacks Callbacks)
{
	AUTORTFM_ASSERT(GNumRegisteredHeapRedirects < GMaxRegisteredHeapRedirects);

	// helper for handling casting the return type to the input function signature
	auto LookupClosedHelper = [](auto InFunction) -> decltype(InFunction)
	{
		using FnType = decltype(InFunction);

		return reinterpret_cast<FnType>(autortfm_lookup_function(reinterpret_cast<void*>(InFunction), "RegisterHeapRedirects"));
	};

	// when loads and stores get redirected, it seems useful to not make a closed->open transition
	// so that the redirect callbacks can abort the transaction, etc. So here in the registration
	// function we translate the function pointers passed in to be their closed variants
	FHeapRedirectCallbacks ClosedCallbacks;
	ClosedCallbacks.AddressSpace = Callbacks.AddressSpace;
	ClosedCallbacks.RedirectedLoad = LookupClosedHelper(Callbacks.RedirectedLoad);
	ClosedCallbacks.RedirectedLoad8 = LookupClosedHelper(Callbacks.RedirectedLoad8);
	ClosedCallbacks.RedirectedLoad4 = LookupClosedHelper(Callbacks.RedirectedLoad4);
	ClosedCallbacks.RedirectedLoad2 = LookupClosedHelper(Callbacks.RedirectedLoad2);
	ClosedCallbacks.RedirectedLoad1 = LookupClosedHelper(Callbacks.RedirectedLoad1);
	ClosedCallbacks.RedirectedStore = LookupClosedHelper(Callbacks.RedirectedStore);
	ClosedCallbacks.RedirectedStore8 = LookupClosedHelper(Callbacks.RedirectedStore8);
	ClosedCallbacks.RedirectedStore4 = LookupClosedHelper(Callbacks.RedirectedStore4);
	ClosedCallbacks.RedirectedStore2 = LookupClosedHelper(Callbacks.RedirectedStore2);
	ClosedCallbacks.RedirectedStore1 = LookupClosedHelper(Callbacks.RedirectedStore1);
	GRegisteredHeapRedirects[GNumRegisteredHeapRedirects++] = ClosedCallbacks;
}

extern "C" void autortfm_on_commit(void (*Work)(void*), void* Arg)
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_on_commit(Work, Arg);
	}

	Work(Arg);
}

extern "C" void autortfm_on_pre_abort(void (*Work)(void*), void* Arg) noexcept
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_on_pre_abort(Work, Arg);
	}
}

extern "C" void autortfm_on_abort(void (*Work)(void*), void* Arg) noexcept
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_on_abort(Work, Arg);
	}
}

extern "C" void autortfm_push_on_abort_handler(const void* Key, void (*Work)(void*), void* Arg) noexcept
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_push_on_abort_handler(Key, Work, Arg);
	}
}

extern "C" void autortfm_pop_on_abort_handler(const void* Key) noexcept
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_pop_on_abort_handler(Key);
	}
}

extern "C" void* autortfm_did_allocate(void* Ptr, size_t Size) noexcept
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_did_allocate(Ptr, Size);
	}

	return Ptr;
}

extern "C" void autortfm_did_free(void* Ptr) noexcept
{
	if (autortfm_is_closed())
	{
		return RTFM_autortfm_did_free(Ptr);
	}

	// We only need to process did free if we need to track allocation locations.
	if constexpr (bTrackAllocationLocations)
	{
		// We only care about frees that are occurring when the transaction
		// is in an on-going state (it's not committing or aborting).
		if (FContext* const Context = FContext::Get(); Context && Context->IsTransactional())
		{
			Context->DidFree(Ptr);
		}
	}
}

// If running with AutoRTFM enabled, then perform an ABI check between the
// AutoRTFM compiler and the AutoRTFM runtime, to ensure that memory is being
// laid out in an identical manner between the AutoRTFM runtime and the AutoRTFM
// compiler pass. Should not be called manually by the user, a call to this will
// be injected by the compiler into a global constructor in the AutoRTFM compiled
// code.
extern "C" UE_AUTORTFM_NOAUTORTFM void autortfm_check_abi(void* const Ptr, const size_t Size) noexcept
{
	struct FConstants final
	{
		const uint32_t Major = AutoRTFM::Constants::Major;
		const uint32_t Minor = AutoRTFM::Constants::Minor;
		const uint32_t Patch = AutoRTFM::Constants::Patch;

		// This is messy - but we want to do comparisons but without comparing any padding bytes.
		// Before C++20 we cannot use a default created operator== and operator!=, so we use this
		// ugly trick to just compare the members.
	private:
		auto Tied() const
		{
			return std::make_tuple(Major, Minor, Patch);
		}

	public:
		bool operator==(const FConstants& Other) const
		{
			return Tied() == Other.Tied();
		}

		bool operator!=(const FConstants& Other) const
		{
			return !(*this == Other);
		}
	} RuntimeConstants;

	AUTORTFM_FATAL_IF(sizeof(FConstants) != Size, "ABI error between AutoRTFM compiler and runtime");

	const FConstants* const CompilerConstants = static_cast<FConstants*>(Ptr);

	AUTORTFM_FATAL_IF(RuntimeConstants != *CompilerConstants, "ABI error between AutoRTFM compiler and runtime");
}
}  // namespace AutoRTFM

#endif  // defined(__AUTORTFM) && __AUTORTFM
