// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "ExternAPI.h"
#include "MemoryStats.h"
#include "Utils.h"

#include <concepts>
#include <utility>

namespace AutoRTFM
{

// A collection of memory allocator static methods which can be passed by
// template argument to AutoRTFM's container types, ensuring that the container
// uses these methods for all heap allocations.
template <typename T>
concept Allocator = requires {
	{
		// The function used to allocate memory from the heap. Must not be null.
		// This API should never return null; if the allocator cannot satisfy the request, it must abort the program.
		// static void* Allocate(size_t Size, size_t Alignment)
		T::Allocate(std::declval<size_t>(), std::declval<size_t>())
	} -> std::same_as<void*>;
	{

		// The function used to reallocate memory from the heap. Must not be null.
		// This API should never return null; if the allocator cannot satisfy the request, it must abort the program.
		// static void* Reallocate(void* Pointer, size_t Size, size_t Alignment, size_t PreviousSize)
		T::Reallocate(std::declval<void*>(), std::declval<size_t>(), std::declval<size_t>(), std::declval<size_t>())
	} -> std::same_as<void*>;
	{

		// The function used to allocate zeroed memory from the heap. Must not be null.
		// This API should never return null; if the allocator cannot satisfy the request, it must abort the program.
		// static void* AllocateZeroed(size_t Size, size_t Alignment)
		T::AllocateZeroed(std::declval<size_t>(), std::declval<size_t>())
	} -> std::same_as<void*>;
	{
		// The function used to free memory allocated by Allocate(), Reallocate() or AllocateZeroed().
		// Must not be null.
		// static void Free(void* Pointer, size_t AllocationSize)
		T::Free(std::declval<void*>(), std::declval<size_t>())
	} -> std::same_as<void>;
};

// The AutoRTFM default Allocator that wraps the external API's memory
// allocation functions.
struct AUTORTFM_INTERNAL FAllocator
{
	static inline void* Allocate(size_t Size, size_t Alignment)
	{
		FMemoryStats::UpdateBytesAllocated(Size);
		return GExternAPI.Allocate(Size, Alignment);
	}

	static inline void* Reallocate(void* Pointer, size_t Size, size_t Alignment, size_t PreviousSize)
	{
		FMemoryStats::UpdateBytesAllocated(static_cast<ptrdiff_t>(Size) - static_cast<ptrdiff_t>(PreviousSize));
		return GExternAPI.Reallocate(Pointer, Size, Alignment, PreviousSize);
	}

	static inline void* AllocateZeroed(size_t Size, size_t Alignment)
	{
		FMemoryStats::UpdateBytesAllocated(Size);
		return GExternAPI.AllocateZeroed(Size, Alignment);
	}

	static inline void Free(void* Pointer, size_t AllocationSize)
	{
		FMemoryStats::UpdateBytesAllocated(-static_cast<ptrdiff_t>(AllocationSize));
		return GExternAPI.Free(Pointer, AllocationSize);
	}
};

static_assert(Allocator<FAllocator>);

// Constructs a new T with the given arguments, using Allocator.
template <typename Allocator, typename T, typename... ARGS>
static inline T* New(ARGS&&... Args)
{
	return new (Allocator::Allocate(sizeof(T), alignof(T))) T(std::forward<ARGS>(Args)...);
}

// Destructs and frees a T* allocated with New<Allocator, T>(...).
// Ptr should be immediately set to null after calling.
template <typename Allocator, typename T, typename... ARGS>
static inline void Delete(T* Ptr)
{
	if (Ptr)
	{
		Ptr->~T();
		Allocator::Free(Ptr, sizeof(T));
	}
}

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
