// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Allocator.h"
#include "Utils.h"

#include <cstddef>

namespace AutoRTFM
{

// A single-threaded, bump memory allocator that sub-allocates out of blocks of
// inline memory and once that's exhausted, from heap allocated memory.
// All allocated memory is freed when the TBlockAllocator is destructed.
// There is no way to free individual allocations.
//
// Template parameters:
//   InlineBlockDataSize - The size of the data section of the inline block.
//   DataAlignment - The maximum supported alignment of data from this allocator.
//   GrowthPercentage - The percentage to grow the block size each reallocation.
template <size_t InlineBlockDataSize = 256, size_t DataAlignment = 16, size_t GrowthPercentage = 200>
class AUTORTFM_INTERNAL TBlockAllocator
{
public:
	// Constructor
	TBlockAllocator() = default;

	// Destructor - frees all the memory allocated by the allocator.
	~TBlockAllocator()
	{
		FreeAll();
	}

	// Frees all allocations made by the allocator.
	void FreeAll()
	{
		while (Tail != &InlineBlock)
		{
			FBlockHeader* Prev = Tail->Prev;
			FAllocator::Free(Tail, Tail->TotalSize());
			Tail = Prev;
		}
		InlineBlock.NextAllocation = reinterpret_cast<std::byte*>(&InlineBlock) + sizeof(FBlockHeader);
		NextBlockSize = InlineBlockDataSize * GrowthPercentage / 100;
	}

	// Allocates memory from the block allocator.
	// Alignment must be a power of two and no larger than DataAlignment.
	inline void* Allocate(size_t Size, size_t Alignment)
	{
		AUTORTFM_ASSERT(Alignment <= DataAlignment);

		if (void* Allocation = Tail->TryAllocate(Size, Alignment))
		{
			return Allocation;
		}

		size_t NewBlockSize = std::max(NextBlockSize, Size);
		NextBlockSize = NextBlockSize * GrowthPercentage / 100;
		FBlockHeader* NewBlock = FBlockHeader::New(Tail, NewBlockSize);
		Tail = NewBlock;

		void* Allocation = NewBlock->TryAllocate(Size, Alignment);
		AUTORTFM_ASSERT(Allocation);
		return Allocation;
	}

private:
	TBlockAllocator(TBlockAllocator&&) = delete;
	TBlockAllocator(const TBlockAllocator&) = delete;
	TBlockAllocator& operator=(const TBlockAllocator&) = delete;
	TBlockAllocator& operator=(TBlockAllocator&&) = delete;

	static constexpr size_t BlockAlignment = std::max<size_t>(DataAlignment, 8);

	struct alignas(BlockAlignment) FBlockHeader
	{
		// The previous FBlock in the singly linked-list
		FBlockHeader* const Prev = nullptr;
		// The size of the block's data
		size_t const BlockDataSize;
		// Pointer to the next sub-allocation.
		std::byte* NextAllocation = reinterpret_cast<std::byte*>(this) + sizeof(FBlockHeader);
		// <data>

		// Allocates, constructs and returns a new block from the heap-allocated memory.
		static FBlockHeader* New(FBlockHeader* Prev, size_t BlockDataSize)
		{
			void* Memory = FAllocator::Allocate(sizeof(FBlockHeader) + BlockDataSize, BlockAlignment);
			return new (Memory) FBlockHeader{Prev, BlockDataSize};
		}

		// Returns the total size in bytes of this block.
		inline size_t TotalSize() const
		{
			return sizeof(FBlockHeader) + BlockDataSize;
		}

		// Attempts to sub-allocate with out of this block. Returns a pointer to
		// the sub-allocated memory on success, or nullptr on failure.
		inline void* TryAllocate(size_t Size, size_t Alignment)
		{
			std::byte* const BlockDataStart = reinterpret_cast<std::byte*>(this) + sizeof(FBlockHeader);
			std::byte* const BlockDataEnd = BlockDataStart + BlockDataSize;
			std::byte* const Ptr = reinterpret_cast<std::byte*>(AlignUp(reinterpret_cast<uintptr_t>(NextAllocation), Alignment));
			std::byte* const End = Ptr + Size;
			if (End > BlockDataEnd || /* overflow */ AUTORTFM_UNLIKELY(End < BlockDataStart))
			{
				return nullptr;
			}
			NextAllocation = End;
			return Ptr;
		}
	};

	static_assert(sizeof(FBlockHeader) % DataAlignment == 0);

	size_t NextBlockSize = InlineBlockDataSize * GrowthPercentage / 100;
	FBlockHeader* Tail = &InlineBlock;
	FBlockHeader InlineBlock{/* Prev */ nullptr, InlineBlockDataSize};
	alignas(BlockAlignment) std::byte Data[InlineBlockDataSize];
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
