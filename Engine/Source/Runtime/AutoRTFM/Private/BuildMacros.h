// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef AUTORTFM_BUILD_DEBUG
#if defined(UE_BUILD_DEBUG) && UE_BUILD_DEBUG
#define AUTORTFM_BUILD_DEBUG 1
#else
#define AUTORTFM_BUILD_DEBUG 0
#endif
#endif

#ifndef AUTORTFM_BUILD_DEVELOPMENT
#if defined(UE_BUILD_DEVELOPMENT) && UE_BUILD_DEVELOPMENT
#define AUTORTFM_BUILD_DEVELOPMENT 1
#else
#define AUTORTFM_BUILD_DEVELOPMENT 0
#endif
#endif

#ifndef AUTORTFM_BUILD_TEST
#if defined(UE_BUILD_TEST) && UE_BUILD_TEST
#define AUTORTFM_BUILD_TEST 1
#else
#define AUTORTFM_BUILD_TEST 0
#endif
#endif

#ifndef AUTORTFM_BUILD_SHIPPING
#if defined(UE_BUILD_SHIPPING) && UE_BUILD_SHIPPING
#define AUTORTFM_BUILD_SHIPPING 1
#else
#define AUTORTFM_BUILD_SHIPPING 0
#endif
#endif

static_assert(AUTORTFM_BUILD_DEBUG + AUTORTFM_BUILD_DEVELOPMENT + AUTORTFM_BUILD_TEST + AUTORTFM_BUILD_SHIPPING == 1);

#ifndef AUTORTFM_PLATFORM_WINDOWS
#if defined(PLATFORM_WINDOWS) && PLATFORM_WINDOWS
#define AUTORTFM_PLATFORM_WINDOWS 1
#else
#define AUTORTFM_PLATFORM_WINDOWS 0
#endif
#endif

#ifndef AUTORTFM_PLATFORM_MAC
#if defined(PLATFORM_MAC) && PLATFORM_MAC
#define AUTORTFM_PLATFORM_MAC 1
#else
#define AUTORTFM_PLATFORM_MAC 0
#endif
#endif

#ifndef AUTORTFM_PLATFORM_LINUX
#if defined(PLATFORM_LINUX) && PLATFORM_LINUX
#define AUTORTFM_PLATFORM_LINUX 1
#else
#define AUTORTFM_PLATFORM_LINUX 0
#endif
#endif

#if defined(__x86_64__) || defined(_M_X64)
#define AUTORTFM_ARCHITECTURE_X64 1
#else
#define AUTORTFM_ARCHITECTURE_X64 0
#endif

#if defined(__has_feature)
#if __has_feature(cxx_exceptions)
#define AUTORTFM_EXCEPTIONS_ENABLED 1
#endif
#endif
#if !defined(AUTORTFM_EXCEPTIONS_ENABLED)
#define AUTORTFM_EXCEPTIONS_ENABLED 0
#endif
