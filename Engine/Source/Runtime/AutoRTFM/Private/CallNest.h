// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM.h"
#include "LongJump.h"

namespace AutoRTFM
{

class FContext;

class AUTORTFM_DISABLE FCallNest final
{
public:
	FCallNest(FCallNest* Parent, TransactionID Transaction) : Parent(Parent), Transaction(Transaction) {}

	FCallNest* const Parent;
	TransactionID const Transaction;
	FLongJump AbortJump;
	ETransactionStatus Status = ETransactionStatus::Executing;

	// Whether this succeeded or not is reflected in Context::GetStatus().
	template <typename TTryFunctor>
	ETransactionStatus Try(const TTryFunctor& TryFunctor);
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
