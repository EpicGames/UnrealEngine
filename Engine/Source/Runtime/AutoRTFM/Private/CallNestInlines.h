// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "CallNest.h"

namespace AutoRTFM
{

template <typename TTryFunctor>
ETransactionStatus FCallNest::Try(const TTryFunctor& TryFunctor)
{
	AbortJump.TryCatch(
		[&]()
		{
			TryFunctor();
			AUTORTFM_ASSERT(Status == ETransactionStatus::Executing);
		},
		[&]() { AUTORTFM_ASSERT(Status != ETransactionStatus::Executing); });
	return Status;
}

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
