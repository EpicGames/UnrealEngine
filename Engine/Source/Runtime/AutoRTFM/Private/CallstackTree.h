// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Allocator.h"
#include "AutoRTFM/CAPI.h"
#include "Stack.h"
#include "StlAllocator.h"
#include "Utils.h"

#include <cstdint>
#include <unordered_set>

namespace AutoRTFM
{

// FCallstackTree is a store for callstacks that internally holds a DAG of
// frames which reduces memory consumption by de-duplicating common base frames.
template <Allocator Allocator = FAllocator>
class AUTORTFM_INTERNAL TCallstackTree final
{
public:
	using FHandle = uint32_t;
	using FCallstack = TStack<autortfm_callstack_frame, 8, Allocator>;
	static constexpr FHandle InvalidHandle = 0;
	static constexpr FHandle MaxHandle = ~InvalidHandle;

	// Adds a callstack consisting of NumFrames frames at Frames.
	// Frames starts with the most deeply-nested frame pointer.
	// Returns a handle that can be passed to Get() to retrieve the callstack.
	FHandle Add(size_t NumFrames, autortfm_callstack_frame const* Frames)
	{
		FHandle Parent = InvalidHandle;
		FHandle Handle = InvalidHandle;
		size_t I = NumFrames;
		while (I--)
		{
			FEntry const Entry{Parent, Frames[I]};
			if (auto It = Lookup.find(Entry); It != Lookup.end())
			{
				Handle = *It;
			}
			else
			{
				Entries.Push(Entry);
				AUTORTFM_ASSERT_DEBUG(Entries.Num() <= MaxHandle);
				Handle = static_cast<FHandle>(Entries.Num());
				Lookup.emplace_hint(It, Handle);
			}

			Parent = Handle;
		}
		return Handle;
	}

	// Retrieves a callstack previously added with Add().
	FCallstack Get(FHandle Handle) const
	{
		FCallstack StackTrace;
		while (Handle != InvalidHandle)
		{
			FEntry const& Entry = Entries[Handle - 1];
			StackTrace.Push(Entry.Frame);
			Handle = Entry.Parent;
		}
		return StackTrace;
	}

private:
	struct FEntry
	{
		FHandle Parent;
		autortfm_callstack_frame Frame;

		bool operator==(const FEntry&) const = default;
	};

	using FEntryList = TStack<FEntry, 32, Allocator>;

	struct FHasher
	{
		using is_transparent = void;
		const FEntryList& Entries;
		size_t operator()(const FEntry& Entry) const
		{
			return (Entry.Parent * 31) ^ Entry.Frame;
		}
		size_t operator()(FHandle Handle) const
		{
			const FEntry& Entry = Entries[Handle - 1];
			return operator()(Entry);
		}
	};

	struct FEquality
	{
		using is_transparent = void;
		const FEntryList& Entries;
		size_t operator()(const FEntry& Entry, FHandle Handle) const
		{
			return Entries[Handle - 1] == Entry;
		}
		size_t operator()(FHandle Handle, const FEntry& Entry) const
		{
			return operator()(Entry, Handle);
		}
		size_t operator()(FHandle A, FHandle B) const
		{
			return A == B;
		}
	};

	FEntryList Entries;
	std::unordered_set<FHandle, FHasher, FEquality, StlAllocator<FHandle, Allocator>> Lookup{
		/* bucket_count */ 32,
		/* hash */ FHasher{Entries},
		/* equal */ FEquality{Entries},
	};
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
