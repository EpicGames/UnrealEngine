// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/CAPI.h"
#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM.h"
#include "BuildMacros.h"
#include "ContextInlines.h"
#include "ExternAPI.h"
#include "FunctionMapInlines.h"
#include "Memcpy.h"
#include "ScopedGuard.h"

namespace AutoRTFM
{

#if !AUTORTFM_BUILD_SHIPPING

// Check for writes to null in development code, so that the inevitable crash will occur
// in the caller's code rather than in the AutoRTFM runtime.
#define UE_AUTORTFM_HANDLE_NULL_WRITE(Ptr)     \
	do                                         \
	{                                          \
		if (AUTORTFM_UNLIKELY(Ptr == nullptr)) \
		{                                      \
			return;                            \
		}                                      \
	} while (0)

#else

// In shipping code, we don't want to spend any cycles on a redundant check.
// We do want the compiler to optimize as if the pointer is non-null, though.
#define UE_AUTORTFM_HANDLE_NULL_WRITE(Ptr) UE_ASSUME(Ptr != nullptr)

#endif

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_record_write(void* Ptr, size_t Size) noexcept
{
	UE_AUTORTFM_HANDLE_NULL_WRITE(Ptr);
	FContext* Context = FContext::Get();
	Context->RecordWrite(Ptr, Size);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_record_write_1(void* Ptr) noexcept
{
	UE_AUTORTFM_HANDLE_NULL_WRITE(Ptr);
	FContext* Context = FContext::Get();
	Context->RecordWrite<1>(Ptr);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_record_write_2(void* Ptr) noexcept
{
	UE_AUTORTFM_HANDLE_NULL_WRITE(Ptr);
	FContext* Context = FContext::Get();
	Context->RecordWrite<2>(Ptr);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_record_write_4(void* Ptr) noexcept
{
	UE_AUTORTFM_HANDLE_NULL_WRITE(Ptr);
	FContext* Context = FContext::Get();
	Context->RecordWrite<4>(Ptr);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_record_write_8(void* Ptr) noexcept
{
	UE_AUTORTFM_HANDLE_NULL_WRITE(Ptr);
	FContext* Context = FContext::Get();
	Context->RecordWrite<8>(Ptr);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_record_masked_write(
	void* Ptr, uintptr_t Mask, int NumMaskBits, int ValueSizeBytes) noexcept
{
	UE_AUTORTFM_HANDLE_NULL_WRITE(Ptr);

	char* IncrementablePtr = static_cast<char*>(Ptr);
	for (int i = 0; i < NumMaskBits; i++)
	{
		if (Mask & (1u << i))
		{
			autortfm_record_write(IncrementablePtr, ValueSizeBytes);
		}

		IncrementablePtr += ValueSizeBytes;
	}
}

// there are two kinds of redirected loads/stores - ones that are explicitly marked up with an address
// space, and others where the AddressSpace == 0 and the information is stored in the bottom nibble
// of the top byte. This helper is used to extract the nibble
static uint32 AutoRTFMExtractAddressSpaceFromAddress(uint64 Address)
{
	return (uint32)((Address >> 56) & 0x0F);
}

static bool AutoRTFMIsPointerNegative(const void* InPointer)
{
	union
	{
		int64 Address;
		const void* Pointer;
	};

	Pointer = InPointer;
	return Address < 0;
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_redirected_load(
	uint32 AddressSpace, void* DestPointer, uint64 Size, uint64 SourceAddress, bool bWillWriteHint) noexcept
{
	ForTheRuntime::RedirectedLoad(AddressSpace ? AddressSpace : AutoRTFMExtractAddressSpaceFromAddress(SourceAddress), DestPointer,
		Size, SourceAddress, bWillWriteHint);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN uint64 autortfm_redirected_load_8(
	uint32 AddressSpace, uint64 SourceAddress, bool bWillWriteHint) noexcept
{
	uint64 Result = 0;
	ForTheRuntime::RedirectedLoad8(
		AddressSpace ? AddressSpace : AutoRTFMExtractAddressSpaceFromAddress(SourceAddress), &Result, SourceAddress, bWillWriteHint);
	return Result;
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN uint32 autortfm_redirected_load_4(
	uint32 AddressSpace, uint64 SourceAddress, bool bWillWriteHint) noexcept
{
	uint32 Result = 0;
	ForTheRuntime::RedirectedLoad4(
		AddressSpace ? AddressSpace : AutoRTFMExtractAddressSpaceFromAddress(SourceAddress), &Result, SourceAddress, bWillWriteHint);
	return Result;
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN uint16 autortfm_redirected_load_2(
	uint32 AddressSpace, uint64 SourceAddress, bool bWillWriteHint) noexcept
{
	uint16 Result = 0;
	ForTheRuntime::RedirectedLoad2(
		AddressSpace ? AddressSpace : AutoRTFMExtractAddressSpaceFromAddress(SourceAddress), &Result, SourceAddress, bWillWriteHint);
	return Result;
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN uint8 autortfm_redirected_load_1(
	uint32 AddressSpace, uint64 SourceAddress, bool bWillWriteHint) noexcept
{
	uint8 Result = 0;
	ForTheRuntime::RedirectedLoad1(
		AddressSpace ? AddressSpace : AutoRTFMExtractAddressSpaceFromAddress(SourceAddress), &Result, SourceAddress, bWillWriteHint);
	return Result;
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_redirected_store(
	uint32 AddressSpace, uint64 DestAddress, uint64 Size, const void* SourcePointer) noexcept
{
	ForTheRuntime::RedirectedStore(
		AddressSpace ? AddressSpace : AutoRTFMExtractAddressSpaceFromAddress(DestAddress), DestAddress, Size, SourcePointer);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_redirected_store_8(
	uint32 AddressSpace, uint64 DestAddress, uint64 Value) noexcept
{
	ForTheRuntime::RedirectedStore8(
		AddressSpace ? AddressSpace : AutoRTFMExtractAddressSpaceFromAddress(DestAddress), DestAddress, &Value);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_redirected_store_4(
	uint32 AddressSpace, uint64 DestAddress, uint32 Value) noexcept
{
	ForTheRuntime::RedirectedStore4(
		AddressSpace ? AddressSpace : AutoRTFMExtractAddressSpaceFromAddress(DestAddress), DestAddress, &Value);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_redirected_store_2(
	uint32 AddressSpace, uint64 DestAddress, uint16 Value) noexcept
{
	ForTheRuntime::RedirectedStore2(
		AddressSpace ? AddressSpace : AutoRTFMExtractAddressSpaceFromAddress(DestAddress), DestAddress, &Value);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API AUTORTFM_NO_ASAN void autortfm_redirected_store_1(
	uint32 AddressSpace, uint64 DestAddress, uint8 Value) noexcept
{
	ForTheRuntime::RedirectedStore1(
		AddressSpace ? AddressSpace : AutoRTFMExtractAddressSpaceFromAddress(DestAddress), DestAddress, &Value);
}

// Note: Internal.aem maps this to itself when called in the closed
extern "C" void* autortfm_lookup_function(void* OriginalFunction, const char* Where) noexcept
{
	AutoRTFM::UnreachableIfClosed();  // AEM uses this as the closed variant, but always uninstrumented.
	return FunctionMapLookup(OriginalFunction, Where);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void autortfm_memcpy(void* Dst, const void* Src, size_t Size) noexcept
{
	FContext* Context = FContext::Get();
	Memcpy(Dst, Src, Size, Context);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void autortfm_memmove(void* Dst, const void* Src, size_t Size) noexcept
{
	FContext* Context = FContext::Get();
	Memmove(Dst, Src, Size, Context);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void autortfm_memset(void* Dst, int Value, size_t Size) noexcept
{
	FContext* Context = FContext::Get();
	Memset(Dst, Value, Size, Context);
}

template <bool bDstIsRedirected, bool bSrcIsRedirected>
AUTORTFM_INTERNAL void autortfm_memmove_redirectable_impl(void* Dst, const void* Src, size_t Size) noexcept
{
	int8 MoveBackwards = (Src < Dst);

	uint8* DstCursor = (uint8*)Dst;
	uint8* SrcCursor = (uint8*)Src;

	if (MoveBackwards)
	{
		DstCursor += Size;
		SrcCursor += Size;
	}

	while (Size >= 8)
	{
		uint64 Data = 0;

		// if we are going backwards, the cursor starts out just beyond the end
		if (MoveBackwards)
		{
			DstCursor -= 8;
			SrcCursor -= 8;
		}

		if (bSrcIsRedirected)
		{
			Data = autortfm_redirected_load_8(0, (uint64)SrcCursor, false);
		}
		else
		{
			// use memcpy as this is potentially an unaligned load
			memcpy(&Data, SrcCursor, 8);
		}

		if (bDstIsRedirected)
		{
			autortfm_redirected_store_8(0, (uint64)DstCursor, Data);
		}
		else
		{
			// use memcpy as this is potentially an unaligned store
			memcpy(DstCursor, &Data, 8);
		}

		if (!MoveBackwards)
		{
			DstCursor += 8;
			SrcCursor += 8;
		}

		Size -= 8;
	}

	while (Size > 0)
	{
		uint8 Data = 0;

		if (MoveBackwards)
		{
			DstCursor -= 1;
			SrcCursor -= 1;
		}

		if (bSrcIsRedirected)
		{
			Data = autortfm_redirected_load_1(0, (uint64)SrcCursor, false);
		}
		else
		{
			Data = *SrcCursor;
		}

		if (bDstIsRedirected)
		{
			autortfm_redirected_store_1(0, (uint64)DstCursor, Data);
		}
		else
		{
			*DstCursor = Data;
		}

		if (!MoveBackwards)
		{
			DstCursor += 1;
			SrcCursor += 1;
		}

		Size -= 1;
	}
}

template <bool bDstIsRedirected>
AUTORTFM_INTERNAL void autortfm_memset_redirectable_impl(void* Dst, uint8 Value, size_t Size) noexcept
{
	uint8* DstCursor = (uint8*)Dst;

	uint64 Data8 = ((uint64)Value) * 0x0101010101010101ull;

	while (Size >= 8)
	{
		if (bDstIsRedirected)
		{
			autortfm_redirected_store_8(0, (uint64)DstCursor, Data8);
		}
		else
		{
			// use memcpy as this is potentially an unaligned store
			memcpy(DstCursor, &Data8, 8);
		}

		DstCursor += 8;
		Size -= 8;
	}

	while (Size > 0)
	{
		uint8 Data = Value;

		if (bDstIsRedirected)
		{
			autortfm_redirected_store_1(0, (uint64)DstCursor, Data);
		}
		else
		{
			*DstCursor = Data;
		}

		DstCursor += 1;
		Size -= 1;
	}
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void autortfm_memmove_redirectable(void* Dst, const void* Src, size_t Size) noexcept
{
	FContext* Context = FContext::Get();

	bool bDstIsRedirected = AutoRTFMIsPointerNegative(Dst);
	bool bSrcIsRedirected = AutoRTFMIsPointerNegative(Src);

	if (bDstIsRedirected)
	{
		if (bSrcIsRedirected)
		{
			autortfm_memmove_redirectable_impl<true, true>(Dst, Src, Size);
		}
		else
		{
			autortfm_memmove_redirectable_impl<true, false>(Dst, Src, Size);
		}
	}
	else
	{
		Context->RecordWrite(Dst, Size);

		if (bSrcIsRedirected)
		{
			autortfm_memmove_redirectable_impl<false, true>(Dst, Src, Size);
		}
		else
		{
			autortfm_memmove_redirectable_impl<false, false>(Dst, Src, Size);
		}
	}
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void autortfm_memcpy_redirectable(void* Dst, const void* Src, size_t Size) noexcept
{
	autortfm_memmove_redirectable(Dst, Src, Size);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void autortfm_memset_redirectable(void* Dst, int Value, size_t Size) noexcept
{
	FContext* Context = FContext::Get();

	bool bDstIsRedirected = AutoRTFMIsPointerNegative(Dst);

	if (bDstIsRedirected)
	{
		autortfm_memset_redirectable_impl<true>(Dst, (uint8)Value, Size);
	}
	else
	{
		Context->RecordWrite(Dst, Size);
		autortfm_memset_redirectable_impl<false>(Dst, (uint8)Value, Size);
	}
}

extern "C" void autortfm_unreachable(const char* Message) noexcept
{
	AUTORTFM_REPORT_ERROR("AutoRTFM Unreachable: %s", Message);
	__builtin_unreachable();
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void autortfm_llvm_fail(const char* Message) noexcept
{
	if (Message)
	{
		AUTORTFM_REPORT_ERROR("AutoRTFM LLVM Failure: %s", Message);
	}
	else
	{
		AUTORTFM_REPORT_ERROR("AutoRTFM LLVM Failure");
	}
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void autortfm_llvm_missing_function() noexcept
{
	if (ForTheRuntime::GetInternalAbortAction() == ForTheRuntime::EAutoRTFMInternalAbortActionState::Crash)
	{
		AUTORTFM_FATAL("Transaction failing because of missing function");
	}
	else
	{
		AUTORTFM_ENSURE_MSG(!ForTheRuntime::GetEnsureOnInternalAbort(), "Transaction failing because of missing function");
	}

	FContext* Context = FContext::Get();
	Context->AbortTransactionAndThrow(ETransactionStatus::AbortedByLanguage);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_FORCENOINLINE UE_AUTORTFM_API void autortfm_called_no_autortfm() noexcept
{
	AUTORTFM_FATAL("inlined UE_AUTORTFM_NOAUTORTFM function called from the closed");
}

}  // namespace AutoRTFM

#endif  // defined(__AUTORTFM) && __AUTORTFM
