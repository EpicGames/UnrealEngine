// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

namespace AutoRTFM
{

// Enumerator used to control validity assertions for containers
// (bounds checking, etc).
enum class EContainerValidation
{
	Enabled,
	Disabled
};

}

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
