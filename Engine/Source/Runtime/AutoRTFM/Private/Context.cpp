// Copyright Epic Games, Inc. All Rights Reserved.

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Context.h"
#include "ContextInlines.h"

#include "Allocator.h"
#include "AutoRTFM.h"
#include "CallNest.h"
#include "CallNestInlines.h"
#include "MetricsPriv.h"
#include "StackRange.h"
#include "ToString.h"
#include "Transaction.h"
#include "TransactionInlines.h"
#include "Utils.h"

#if AUTORTFM_PLATFORM_WINDOWS
#include "WindowsHeader.h"
#endif

namespace AutoRTFM
{

FContext* FContext::Instance = nullptr;

FContext* FContext::Create()
{
	AUTORTFM_ENSURE(Instance == nullptr);
	void* Memory = FAllocator::Allocate(sizeof(FContext), alignof(FContext));
	Instance = new (Memory) FContext();
	return Instance;
}

void FContext::Destroy()
{
	if (FContext* Context = Instance)
	{
		// Null instance before deleting the context, as freeing the context may
		// attempt to use the context.
		Instance = nullptr;
		Context->~FContext();
		FAllocator::Free(Context, sizeof(FContext));
	}
	else
	{
		AUTORTFM_FATAL("FContext::Destroy() called twice");
	}
}

FContext::~FContext()
{
#if AUTORTFM_SANITIZER
	if (size_t NumSanitizerIssues = Sanitizer::NumberOfIssuesFound())
	{
		AUTORTFM_WARN("AutoRTFMSan: %zu sanitizer issues detected", NumSanitizerIssues);
	}
#endif
}

void FContext::StartTransaction()
{
	AUTORTFM_ASSERT(Status == EContextStatus::OnTrack);
	AUTORTFM_ENSURE_MSG(CurrentMaterializedTransaction, "FContext::StartTransaction() cannot be used for the outermost transaction");

	NumDeferredTransactions += 1;

	IncrementNumTransactionsStartedMetric();
}

void FContext::MaterializeDeferredTransactions()
{
	if (uint64_t NumToAllocate = NumDeferredTransactions; NumToAllocate > 0)
	{
		// This form of transaction is always ultimately within a scoped Transact
		AUTORTFM_ASSERT(Status == EContextStatus::OnTrack);
		AUTORTFM_ENSURE_MSG(CurrentMaterializedTransaction,
			"FContext::MaterializeDeferredTransactions() can only be called within a scoped transaction");
		NumDeferredTransactions = 0;
		for (uint64_t I = 0; I < NumToAllocate; ++I)
		{
			PushTransaction(CurrentMaterializedTransaction->GetStackRange());
		}
	}
}

void FContext::CommitTransaction()
{
	AUTORTFM_ASSERT(Status == EContextStatus::OnTrack);

	IncrementNumTransactionsCommittedMetric();

	if (NumDeferredTransactions)
	{
		NumDeferredTransactions--;
		return;
	}

	AUTORTFM_ASSERT(CurrentMaterializedTransaction);

	Status = EContextStatus::Committing;
	{
		SetCallNestTransactionNonCascadingStatus(ETransactionStatus::Committed);
		CurrentMaterializedTransaction->Commit();
		PopTransaction();
	}
	Status = EContextStatus::OnTrack;
}

void FContext::AbortTransaction(ETransactionStatus Reason)
{
	AUTORTFM_ASSERT(Status == EContextStatus::OnTrack);
	AUTORTFM_ASSERT(HasAborted(Reason));
	AUTORTFM_ASSERT(CurrentMaterializedTransaction);

	switch (Reason)
	{
		case ETransactionStatus::AbortedByLanguage:
			IncrementNumTransactionalViolationsMetric();
			break;
		case ETransactionStatus::AbortedByRequest:
			IncrementNumRequestedAbortsMetric();
			break;
		case ETransactionStatus::AbortedByCascadingAbort:
			IncrementNumRequestedCascadingAbortsMetric();
			break;
		case ETransactionStatus::AbortedByCascadingRetry:
			IncrementNumRequestedCascadingRetriesMetric();
			break;
		[[unlikely]] default:
			AUTORTFM_FATAL("invalid Reason in AbortTransaction: %d", static_cast<int>(Reason));
			break;
	}

	Status = EContextStatus::Aborting;

	if (IsCascading(Reason))
	{
		// Set the status of all the call nests with the cascading failure.
		SetCallNestTransactionCascadingStatus(Reason);

		// Cancel out any deferred transactions.
		IncrementNumTransactionsAbortedMetric(NumDeferredTransactions);
		NumDeferredTransactions = 0;

		// Immediately abort and pop the entire transaction stack.
		while (CurrentMaterializedTransaction)
		{
			IncrementNumTransactionsAbortedMetric();
			CurrentMaterializedTransaction->Abort();
			PopTransaction();
		}

		Status = EContextStatus::Unwinding;
	}
	else
	{
		if (NumDeferredTransactions > 0)
		{
			// The topmost transaction was never materialized, so an abort is trivial.
			NumDeferredTransactions--;
			IncrementNumTransactionsAbortedMetric();
		}
		else
		{
			// Set the status of all the call nests associated with the transaction
			// with the (non-cascading) failure.
			SetCallNestTransactionNonCascadingStatus(Reason);

			// Abort and pop the top most transaction.
			IncrementNumTransactionsAbortedMetric();
			CurrentMaterializedTransaction->Abort();
			PopTransaction();
		}

		Status = EContextStatus::OnTrack;
	}
}

void FContext::SetCallNestTransactionCascadingStatus(ETransactionStatus TransactionStatus)
{
	AUTORTFM_ASSERT_DEBUG(IsCascading(TransactionStatus));
	for (FCallNest* Nest = CurrentNest; Nest; Nest = Nest->Parent)
	{
		Nest->Status = TransactionStatus;
	}
}

void FContext::SetCallNestTransactionNonCascadingStatus(ETransactionStatus TransactionStatus)
{
	AUTORTFM_ASSERT_DEBUG(!IsCascading(TransactionStatus));
	AUTORTFM_ASSERT(CurrentMaterializedTransaction);

	for (FCallNest* Nest = CurrentNest; Nest; Nest = Nest->Parent)
	{
		if (Nest->Transaction == CurrentMaterializedTransaction->Identifier())
		{
			Nest->Status = TransactionStatus;
		}
	}
}

void FContext::AbortTransactionAndThrow(ETransactionStatus Reason)
{
	AbortTransaction(Reason);
	Throw();
}

ETransactionStatus FContext::CallClosedNest(void (*ClosedFunction)(void* Arg), void* Arg)
{
	FTransaction* const Transaction = GetCurrentTransaction();
	AUTORTFM_ASSERT(Transaction != nullptr);

	PushCallNest();

	ETransactionStatus TransactionStatus = CurrentNest->Try([&] { ClosedFunction(Arg); });

	PopCallNest();

	return TransactionStatus;
}

void FContext::PushCallNest()
{
	MaterializeDeferredTransactions();
	CurrentNest = CallNestPool.Take(CurrentNest, CurrentMaterializedTransaction->Identifier());
}

void FContext::PopCallNest()
{
	AUTORTFM_ASSERT(CurrentNest != nullptr);
	FCallNest* OldCallNest = CurrentNest;
	CurrentNest = CurrentNest->Parent;

	CallNestPool.Return(OldCallNest);
}

FTransaction* FContext::PushTransaction(FStackRange StackRange)
{
	AUTORTFM_ASSERT(NumDeferredTransactions == 0);

	FTransaction* NewTransaction = TransactionPool.Take(this);
	NewTransaction->Initialize(
		/* Parent */ CurrentMaterializedTransaction,
		/* StackRange */ StackRange);

	CurrentMaterializedTransaction = NewTransaction;

	return NewTransaction;
}

void FContext::PopTransaction()
{
	AUTORTFM_ASSERT(NumDeferredTransactions == 0);
	AUTORTFM_ASSERT(CurrentMaterializedTransaction != nullptr);
	FTransaction* OldTransaction = CurrentMaterializedTransaction;
	CurrentMaterializedTransaction = CurrentMaterializedTransaction->GetParent();
	TransactionPool.Return(OldTransaction);
}

AutoRTFM::FStackRange FContext::GetThreadStackRange()
{
	// On some platforms, looking up the stack range is quite expensive, so caching it
	// is important for performance. Linux glibc is particularly bad--see
	// https://github.com/golang/go/issues/68587 for a deep dive.
	thread_local FStackRange CachedStackRange = []
	{
		FStackRange Stack;

#if AUTORTFM_PLATFORM_WINDOWS
		GetCurrentThreadStackLimits(reinterpret_cast<PULONG_PTR>(&Stack.Low), reinterpret_cast<PULONG_PTR>(&Stack.High));
#elif defined(__APPLE__)
		Stack.High = pthread_get_stackaddr_np(pthread_self());
		size_t StackSize = pthread_get_stacksize_np(pthread_self());
		Stack.Low = static_cast<char*>(Stack.High) - StackSize;
#else
		pthread_attr_t Attr{};

		{
			const int Error = pthread_getattr_np(pthread_self(), &Attr);
			AUTORTFM_ASSERT(0 == Error);
		}

		Stack.Low = 0;
		size_t StackSize = 0;
		pthread_attr_getstack(&Attr, &Stack.Low, &StackSize);
		Stack.High = static_cast<char*>(Stack.Low) + StackSize;

		{
			const int Error = pthread_attr_destroy(&Attr);
			AUTORTFM_ASSERT(0 == Error);
		}
#endif

		AUTORTFM_ASSERT(Stack.High > Stack.Low);
		return Stack;
	}();

	return CachedStackRange;
}

// If exceptions are enabled, then ensure that the transaction is automatically committed if
// an exception is thrown inside the transaction and the exception handler is outside the
// transaction.
struct FContext::FAutoCommitOnException final
{
	// Constructor
	FAutoCommitOnException(FContext& Context) : Context{Context} {}

	// Destructor.
	// If NoExceptionsRaised() has not been called before the destructor, then
	// it is assumed that an exception has been raised.
	~FAutoCommitOnException()
	{
		if (bEnabled)
		{
			Context.CommitTransaction();
			Context.PopCallNest();
			Context.EndScopedTransaction();
		}
	}

	// Called after the transactional block completes execution without throwing
	// an exception. Disables the auto-commit logic in the destructor.
	void NoExceptionsRaised()
	{
		bEnabled = false;
	}

private:
	FContext& Context;
	bool bEnabled = true;
};

ETransactionResult FContext::Transact(void (*UninstrumentedFunction)(void*), void (*InstrumentedFunction)(void*), void* Arg)
{
	switch (Status)
	{
		[[unlikely]] case EContextStatus::Unwinding:
			return ETransactionResult::RejectedTransactDuringUnwind;
		[[unlikely]] case EContextStatus::Committing:
			return ETransactionResult::RejectedTransactDuringCommit;
		[[unlikely]] case EContextStatus::Aborting:
			return ETransactionResult::RejectedTransactDuringAbort;
		[[unlikely]] case EContextStatus::Retrying:
			return ETransactionResult::RejectedTransactDuringRetry;
		[[unlikely]] case EContextStatus::Completing:
			return ETransactionResult::RejectedTransactDuringCompletion;

		case EContextStatus::Idle:
		case EContextStatus::OnTrack:
		case EContextStatus::InStaticLocalInitializer:
			break;
	}

	if (!InstrumentedFunction)
	{
		AUTORTFM_WARN("Could not find function in AutoRTFM::FContext::Transact");
		return ETransactionResult::AbortedByLanguage;
	}

	// TODO: We could do better if we ever need to. There is no fundamental
	// reason we can't have a "range" of deferred transactions in the middle
	// of the transaction stack.
	MaterializeDeferredTransactions();
	AUTORTFM_ASSERT(NumDeferredTransactions == 0);

	IncrementNumTransactionsStartedMetric();

	if (CurrentMaterializedTransaction)
	{
		// Nested transaction
		AUTORTFM_ASSERT(Status == EContextStatus::OnTrack);
		AUTORTFM_ASSERT(CurrentThreadId == FThreadID::GetCurrent());
	}
	else
	{
		// Outer-most transaction
		AUTORTFM_ASSERT(CurrentThreadId == FThreadID::Invalid);
		AUTORTFM_ASSERT(Status == EContextStatus::Idle);
		AUTORTFM_ASSERT(Stack == FStackRange{});
		CurrentThreadId = FThreadID::GetCurrent();
		Stack = GetThreadStackRange();
	}

	char TransactStackStart;
	AUTORTFM_ASSERT(Stack.Contains(&TransactStackStart));
	FStackRange const StackRange{Stack.Low, &TransactStackStart};

	FTransaction* const ParentTransaction = CurrentMaterializedTransaction;

	ETransactionResult Result{};

	bool const bShouldRetryTransaction = CurrentMaterializedTransaction
										   ? AutoRTFM::ForTheRuntime::ShouldRetryNestedTransactionsToo()
										   : AutoRTFM::ForTheRuntime::ShouldRetryNonNestedTransactions();
	bool bTriedToRunOnce = false;

	for (;;)
	{
		Status = EContextStatus::OnTrack;
		AUTORTFM_ASSERT(CurrentMaterializedTransaction == ParentTransaction);

		FTransaction* const Transaction = PushTransaction(StackRange);
		PushCallNest();
		ETransactionStatus TransactionStatus;
		{
			FAutoCommitOnException AutoCommitter{*this};
			TransactionStatus = CurrentNest->Try([&]() { InstrumentedFunction(Arg); });
			AutoCommitter.NoExceptionsRaised();
		}
		PopCallNest();

		if (AUTORTFM_UNLIKELY(bShouldRetryTransaction) && !bTriedToRunOnce && TransactionStatus == ETransactionStatus::Executing)
		{
			AUTORTFM_ASSERT(CurrentMaterializedTransaction == Transaction);
			bTriedToRunOnce = true;
			AbortTransaction(ETransactionStatus::AbortedByRequest);
			if (!ParentTransaction)
			{
				CallCompletionTasks();
			}
			continue;  // Retry transaction
		}

		switch (TransactionStatus)
		{
			case ETransactionStatus::Executing:
				AUTORTFM_ASSERT(CurrentMaterializedTransaction == Transaction);
				CommitTransaction();
				Result = ETransactionResult::Committed;
				break;

			case ETransactionStatus::Committed:  // Transaction was committed in the open.
				Result = ETransactionResult::Committed;
				break;

			case ETransactionStatus::AbortedByRequest:
				Result = ETransactionResult::AbortedByRequest;
				break;

			case ETransactionStatus::AbortedByLanguage:
				Result = ETransactionResult::AbortedByLanguage;
				break;

			case ETransactionStatus::AbortedByCascadingAbort:
				AUTORTFM_ASSERT(CurrentMaterializedTransaction == nullptr);
				if (CurrentNest)
				{
					Throw();
				}
				Result = ETransactionResult::AbortedByCascadingAbort;
				break;

			case ETransactionStatus::AbortedByCascadingRetry:
				AUTORTFM_ASSERT(CurrentMaterializedTransaction == nullptr);
				if (CurrentNest)
				{
					Throw();
				}

				if (!ForTheRuntime::IsAutoRTFMRuntimeEnabled())
				{
					// AutoRTFM runtime was disabled during an abort callback.
					// Execute the function without AutoRTFM as a fallback.
					UninstrumentedFunction(Arg);
					Result = ETransactionResult::Committed;
					break;
				}

				// Retry the transaction
				Status = EContextStatus::Retrying;
				CompletionTasks.Reset();
				RetryTasks.RemoveEachForward([](TTask<void()>& Task) { Task(); });

				// A retry is treated as a new top-level transaction.
				IncrementNumTransactionsStartedMetric();
				continue;

			default:
				AUTORTFM_FATAL("Invalid transaction status: %s", ToString(TransactionStatus));
				break;
		}

		break;
	}

	EndScopedTransaction();

	return Result;
}

void FContext::EndScopedTransaction()
{
	AUTORTFM_ASSERT(Status == EContextStatus::OnTrack || Status == EContextStatus::Unwinding);
	if (CurrentMaterializedTransaction == nullptr)
	{
		// No more transactions in flight.
		RetryTasks.Reset();
		CallCompletionTasks();
		Reset();
	}
}

void FContext::CallCompletionTasks()
{
	AUTORTFM_ASSERT(CurrentMaterializedTransaction == nullptr);
	AUTORTFM_ASSERT(Status == EContextStatus::OnTrack || Status == EContextStatus::Unwinding);

	{
		AUTORTFM_SCOPED_ASSIGNMENT(Status, EContextStatus::Completing);
		CompletionTasks.RemoveEachForward([](TTask<void()>& Task) { Task(); });
	}
}

void FContext::Reset()
{
	AUTORTFM_ASSERT(CurrentThreadId == FThreadID::GetCurrent() || CurrentThreadId == FThreadID::Invalid);

	CurrentThreadId = FThreadID::Invalid;
	Stack = {};
	CurrentMaterializedTransaction = nullptr;
	CurrentNest = nullptr;
	Status = EContextStatus::Idle;
	StackLocalInitializerDepth = 0;
	RetryTasks.Reset();
	CompletionTasks.Reset();
	TaskPool.Reset();
}

void FContext::MaybeThrow()
{
	if (CurrentNest->Status != ETransactionStatus::Executing)
	{
		Throw();
	}
}

void FContext::Throw()
{
	CurrentNest->AbortJump.Throw();
}

}  // namespace AutoRTFM

#endif  // defined(__AUTORTFM) && __AUTORTFM
