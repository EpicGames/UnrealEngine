// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM/Defines.h"
#include "AutoRTFM/Task.h"
#include "CallNest.h"
#include "IntrusivePool.h"
#include "Pool.h"
#include "StackRange.h"
#include "TaskArray.h"
#include "ThreadID.h"
#include "Transaction.h"
#include "Utils.h"

namespace AutoRTFM
{

class AUTORTFM_INTERNAL FContext final
{
	using FTaskPool = typename TTaskArray<TTask<void()>>::FEntryPool;
	using FTransactionPool = TIntrusivePool<FTransaction, 16>;
	using FCallNestPool = TPool<FCallNest, 16>;

public:
	static FContext* Create();
	static FContext* Get();
	static void Destroy();

	// This is public API
	ETransactionResult Transact(void (*UninstrumentedFunction)(void*), void (*InstrumentedFunction)(void*), void* Arg);

	ETransactionStatus CallClosedNest(void (*ClosedFunction)(void* Arg), void* Arg);

	// Begins an unscoped transaction.
	// It is common for a transaction to be started and then immediately aborted
	// without actually being used. As a performance optimization,
	// StartTransaction() increments NumDeferredTransactions, which represents
	// the number of "deferred" transactions at the top of the transaction
	// stack. Deferred transactions will be lazily backed by an FTransaction on
	// the first call to MaterializeDeferredTransactions() or
	// GetCurrentTransaction().
	void StartTransaction();

	// Ends a scoped or unscoped transaction, undoing all writes recorded by the
	// transaction and calling all registered on-abort handlers.
	// If the current transaction is materialized, then the FTransaction is
	// popped from the transaction stack, otherwise NumDeferredTransactions is
	// decremented.
	void AbortTransaction(ETransactionStatus Reason);

	// Convenience function for calling AbortTransaction() followed by Throw().
	void AbortTransactionAndThrow(ETransactionStatus Reason);

	// Ends a scoped or unscoped transaction, calling all registered on-commit
	// handlers.
	// If the current transaction is materialized, then the FTransaction is
	// popped from the transaction stack, otherwise NumDeferredTransactions is
	// decremented.
	void CommitTransaction();

	// Throws if the current (most-nested) FCallNest has a status that is not
	// ETransactionStatus::Executing.
	void MaybeThrow();

	// Record that a write is about to occur at the given LogicalAddress of Size bytes.
	void RecordWrite(void* LogicalAddress, size_t Size);
	template <unsigned SIZE>
	void RecordWrite(void* LogicalAddress);
	template <unsigned SIZE>
	void RecordWriteSlow(void* LogicalAddress);

	void DidAllocate(void* LogicalAddress, size_t Size);
	void DidFree(void* LogicalAddress);

	inline void DeferUntilComplete(TTask<void()>&& Callback);
	inline void DeferUntilRetry(TTask<void()>&& Callback);

	// The number of deferred transactions that have yet to be materialized into
	// a FTransaction.
	inline uint64_t GetNumDeferredTransactions() const
	{
		return NumDeferredTransactions;
	}

	// Returns true if the current (most-nested) transaction is deferred.
	// For more information about deferred transactions, see StartTransaction()
	// and AbortTransaction().
	inline bool MustMaterializeDeferredTransactions() const
	{
		return (AUTORTFM_UNLIKELY(0 < GetNumDeferredTransactions()));
	}

	// Returns the current (most-nested) transaction, materializing the
	// transaction stack if one or more transactions are deferred.
	// For more information about deferred transactions, see StartTransaction()
	// and AbortTransaction().
	inline FTransaction* GetCurrentTransaction()
	{
		if (MustMaterializeDeferredTransactions())
		{
			MaterializeDeferredTransactions();
		}
		return CurrentMaterializedTransaction;
	}

	// Returns the most-nested materialized transaction.
	// For more information about deferred transactions, see StartTransaction()
	// and AbortTransaction().
	inline FTransaction* GetMaterializedTransaction()
	{
		return CurrentMaterializedTransaction;
	}

	// Returns true if the caller's thread is the same as the thread associated
	// with the transaction stack.
	AUTORTFM_ENABLE inline bool IsContextThread() const
	{
		return CurrentThreadId == FThreadID::GetCurrent();
	}

	AUTORTFM_ENABLE inline EContextStatus GetStatus() const
	{
		return IsContextThread() ? Status : EContextStatus::Idle;
	}

	AUTORTFM_ENABLE inline bool IsTransactional() const
	{
		return GetStatus() == EContextStatus::OnTrack;
	}

	inline void EnteringStaticLocalInitializer()
	{
		if (AUTORTFM_LIKELY(GetStatus() == EContextStatus::Idle))
		{
			return;
		}

		if (Status == EContextStatus::OnTrack)
		{
			AUTORTFM_ASSERT(0 == StackLocalInitializerDepth);
			Status = EContextStatus::InStaticLocalInitializer;
			StackLocalInitializerDepth++;
		}
		else if (Status == EContextStatus::InStaticLocalInitializer)
		{
			StackLocalInitializerDepth++;
		}
	}

	inline void LeavingStaticLocalInitializer()
	{
		if (AUTORTFM_LIKELY(GetStatus() == EContextStatus::Idle))
		{
			return;
		}

		AUTORTFM_ASSERT(Status != EContextStatus::OnTrack);

		if (Status == EContextStatus::InStaticLocalInitializer)
		{
			StackLocalInitializerDepth--;

			if (0 == StackLocalInitializerDepth)
			{
				Status = EContextStatus::OnTrack;
			}
		}
	}

	inline FTaskPool& GetTaskPool()
	{
		return TaskPool;
	}

	inline FTransactionPool& GetTransactionPool()
	{
		return TransactionPool;
	}

	// Returns the stack address limits for the caller thread.
	static FStackRange GetThreadStackRange();

private:
	struct FAutoCommitOnException;

	UE_AUTORTFM_API static FContext* Instance;

	FContext()
	{
		Reset();
	}
	FContext(const FContext&) = delete;
	~FContext();

	void MaterializeDeferredTransactions();

	void PushCallNest();
	void PopCallNest();

	FTransaction* PushTransaction(FStackRange);
	void PopTransaction();

	[[noreturn]] void Throw();

	void EndScopedTransaction();
	void CallCompletionTasks();

	// Set the status of all the call nests with the given transaction status.
	void SetCallNestTransactionCascadingStatus(ETransactionStatus Status);

	// Set the status of all the call nests associated with the transaction with
	// the given transaction status.
	void SetCallNestTransactionNonCascadingStatus(ETransactionStatus Status);

	void Reset();

	// We defer allocating FTransactions at the top of the transaction stack.
	// This allows us to make starting a transaction in the open a load, some math, and a store.
	uint64_t NumDeferredTransactions{0};
	FTransaction* CurrentMaterializedTransaction{nullptr};
	FCallNest* CurrentNest{nullptr};

	FStackRange Stack;
	EContextStatus Status{EContextStatus::Idle};
	FThreadID CurrentThreadId;
	uint32_t StackLocalInitializerDepth = 0;
	FTaskPool TaskPool;
	FTransactionPool TransactionPool;
	FCallNestPool CallNestPool;

	TTaskArray<TTask<void()>> CompletionTasks{TaskPool};
	TTaskArray<TTask<void()>> RetryTasks{TaskPool};
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
