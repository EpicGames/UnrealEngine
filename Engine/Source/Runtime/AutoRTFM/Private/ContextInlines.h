// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM/Defines.h"
#include "Context.h"
#include "TransactionInlines.h"

namespace AutoRTFM
{

UE_AUTORTFM_FORCEINLINE FContext* FContext::Get()
{
	return FContext::Instance;
}

AUTORTFM_NO_ASAN UE_AUTORTFM_FORCEINLINE void FContext::RecordWrite(void* LogicalAddress, size_t Size)
{
	GetCurrentTransaction()->RecordWrite(LogicalAddress, Size);
}

template <unsigned SIZE>
AUTORTFM_NO_ASAN UE_AUTORTFM_FORCEINLINE void FContext::RecordWrite(void* LogicalAddress)
{
	if (MustMaterializeDeferredTransactions())
	{
		AUTORTFM_MUST_TAIL return RecordWriteSlow<SIZE>(LogicalAddress);
	}
	else
	{
		GetMaterializedTransaction()->RecordWrite<SIZE>(LogicalAddress);
	}
}

template <unsigned SIZE>
AUTORTFM_NO_ASAN UE_AUTORTFM_FORCENOINLINE void FContext::RecordWriteSlow(void* LogicalAddress)
{
	// Going through get current transaction will do the materialization of deferred transactions, but it is slow
	// so that is why we use this carve out.
	GetCurrentTransaction()->RecordWrite<SIZE>(LogicalAddress);
}

UE_AUTORTFM_FORCEINLINE void FContext::DidAllocate(void* LogicalAddress, size_t Size)
{
	GetCurrentTransaction()->DidAllocate(LogicalAddress, Size);
}

UE_AUTORTFM_FORCEINLINE void FContext::DidFree(void* LogicalAddress)
{
	// We can do free's in the open within a transaction *during* when the
	// transaction itself is being destroyed, so we need to check for that case.
	FTransaction* Transaction = GetCurrentTransaction();
	if (AUTORTFM_LIKELY(Transaction))
	{
		Transaction->DidFree(LogicalAddress);
	}
}

UE_AUTORTFM_FORCEINLINE void FContext::DeferUntilComplete(TTask<void()>&& Callback)
{
	AUTORTFM_ASSERT(Status == EContextStatus::OnTrack);

	// We explicitly must copy the function here because the original was allocated
	// within a transactional context, and thus the memory is allocating under
	// transactionalized conditions. By copying, we create an open copy of the callback.
	TTask<void()> Copy(Callback);
	CompletionTasks.Add(std::move(Copy));
}

UE_AUTORTFM_FORCEINLINE void FContext::DeferUntilRetry(TTask<void()>&& Callback)
{
	AUTORTFM_ASSERT(Status == EContextStatus::OnTrack);

	// We explicitly must copy the function here because the original was allocated
	// within a transactional context, and thus the memory is allocating under
	// transactionalized conditions. By copying, we create an open copy of the callback.
	TTask<void()> Copy(Callback);
	RetryTasks.Add(std::move(Copy));
}

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
