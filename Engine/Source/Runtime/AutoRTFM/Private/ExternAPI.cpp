// Copyright Epic Games, Inc. All Rights Reserved.

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "ExternAPI.h"

#include <cstdio>
#include <memory>

namespace AutoRTFM
{

namespace
{

// A default implementation for GExternAPI.Log().
// Most of GExternAPI is intentionally null, as we do not intend to be running
// AutoRTFM logic before initialization - however there are a number of
// AutoRTFM configuration setters that can be called before initialization
// which have logic that can (potentially) log. Logging is unlikely to be called
// before initialization, but a default printf implementation is preferable to
// crashing due to a nullptr.
void DefaultLog(const char* File, int Line, [[maybe_unused]] void* ProgramCounter, autortfm_log_severity Severity,
	const char* Format, va_list Args)
{
	static constexpr int InlineBufferLength = 256;
	char InlineBuffer[InlineBufferLength];

	va_list Args2;
	va_copy(Args2, Args);
	int Count = vsnprintf(InlineBuffer, InlineBufferLength, Format, Args);
	auto AllocatedBuffer = (Count >= InlineBufferLength) ? std::unique_ptr<char[]>{new char[Count + 1]} : nullptr;
	if (AllocatedBuffer)
	{
		vsnprintf(AllocatedBuffer.get(), Count + 1, Format, Args2);
	}
	va_end(Args2);

	char* const Buffer = AllocatedBuffer ? AllocatedBuffer.get() : InlineBuffer;

	switch (Severity)
	{
		case autortfm_log_verbose:
			fprintf(stdout, "AutoRTFM %s:%d [VERBOSE]: %s", File, Line, Buffer);
			break;
		case autortfm_log_info:
			fprintf(stdout, "AutoRTFM %s:%d [INFO]: %s", File, Line, Buffer);
			break;
		case autortfm_log_warn:
			fprintf(stdout, "AutoRTFM %s:%d [WARN]: %s", File, Line, Buffer);
			break;
		case autortfm_log_error:
			fprintf(stderr, "AutoRTFM %s:%d [ERROR]: %s", File, Line, Buffer);
			break;
		case autortfm_log_fatal:
			fprintf(stderr, "AutoRTFM %s:%d [FATAL]: %s", File, Line, Buffer);
			__builtin_trap();
	}
}

}  // anonymous namespace

autortfm_extern_api GExternAPI = {
	/* Allocate */ nullptr,
	/* Reallocate */ nullptr,
	/* AllocateZeroed */ nullptr,
	/* Free */ nullptr,
	/* Log */ DefaultLog,
	/* LogWithCallstack */ nullptr,
	/* EnsureFailure */ nullptr,
	/* IsLogActive */ nullptr,
	/* CaptureCallstack */ nullptr,
	/* LogCallstack */ nullptr,
	/* OnRuntimeEnabledChanged */ nullptr,
	/* OnRetryTransactionsChanged */ nullptr,
#if AUTORTFM_SANITIZER
	/* OnSanitizerModeChanged */ nullptr,
	/* OnSanitizerRecordClosedWriteCallstacksChanged */ nullptr,
#endif
	/* RollbackWrite */ nullptr,
};

}  // namespace AutoRTFM

#endif  // defined(__AUTORTFM) && __AUTORTFM
