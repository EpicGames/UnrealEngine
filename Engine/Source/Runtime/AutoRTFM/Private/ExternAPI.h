// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM/CAPI.h"

namespace AutoRTFM
{

UE_AUTORTFM_API extern autortfm_extern_api GExternAPI;

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
