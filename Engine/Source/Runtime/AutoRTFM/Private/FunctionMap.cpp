// Copyright Epic Games, Inc. All Rights Reserved.

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "FunctionMap.h"

#include "AutoRTFM.h"
#include "AutoRTFM/ReportHazard.h"
#include "BuildMacros.h"
#include "ContextInlines.h"
#include "FunctionMapInlines.h"
#include "TransactionInlines.h"
#include "Utils.h"

#include <cinttypes>
#include <cstddef>
#include <iterator>
#include <map>
#include <mutex>
#include <stdlib.h>
#include <string.h>

#if AUTORTFM_PLATFORM_WINDOWS
#include "WindowsHeader.h"
#elif AUTORTFM_PLATFORM_LINUX
#include <pthread.h>
#endif

namespace AutoRTFM
{

namespace
{

// The function map holds a hash map of open function pointer to closed function
// pointer. The function map must be obtained by calling FFunctionMap::Get(),
// which holds an internal mutex, preventing concurrent access.
class FFunctionMap
{
	// The internal hash map load factor.
	// Currently capacity must be at least twice the number of entries.
	static constexpr size_t LoadFactor = 2;

	// The number of bits in a size_t integer.
	static constexpr size_t NumBitsInSizeT = sizeof(size_t) * 8;

	// The internal hash map data.
	struct FHashMap
	{
		// The number of elements in the map.
		size_t EntryCount;
		// The map capacity as a power-of-two; that is, a Capacity2n of 10 corresponds to 1024 entries, as 2^10 == 1024.
		size_t Capacity2n;
		// The map capacity minus one, for use as a bitmask; a function map with 1024 elements will have an IndexMask of 0x3FF.
		size_t IndexMask;
		// Actually contains `2 ^ Capacity2n` elements.
		autortfm_open_to_closed_mapping Entries[1];

		static FHashMap* Allocate(size_t Capacity2n)
		{
			size_t TotalSize = offsetof(FHashMap, Entries) + (sizeof(autortfm_open_to_closed_mapping) << Capacity2n);
			FHashMap* Result = static_cast<FHashMap*>(calloc(TotalSize, 1));
			Result->Capacity2n = Capacity2n;
			Result->IndexMask = (1 << Capacity2n) - 1;
			return Result;
		}
	};

	// Returns the number of bits required to represent the given value (must be non-zero)
	static inline size_t NumberOfBits(size_t Value)
	{
		return NumBitsInSizeT - __builtin_clzll(Value);
	}

public:
	// Obtains the FFunctionMap instance.
	static FFunctionMap Get()
	{
		struct FHashMapAndMutex
		{
			FHashMap* Data = FHashMap::Allocate(/* Capacity2n */ 10);
			std::mutex Mutex{};
		};
		static FHashMapAndMutex HashMapAndMutex;
		return FFunctionMap{HashMapAndMutex.Data, HashMapAndMutex.Mutex};
	}

	~FFunctionMap()
	{
		Mutex.unlock();
	}

	// Returns the total number of entries in the function map.
	inline size_t Count() const
	{
		return Map->EntryCount;
	}

	// Ensures that the function map is large enough to hold Count * LoadFactor entries.
	void Reserve(size_t Count)
	{
		const size_t NewCapacity = Count * LoadFactor;
		const size_t NewCapacity2n = NumberOfBits(NewCapacity);
		if (Map->Capacity2n >= NewCapacity2n)
		{
			return;
		}

		FHashMap* NewMap = FHashMap::Allocate(NewCapacity2n);
		for (autortfm_open_to_closed_mapping *OldEntry = Map->Entries, *FinalEntry = Map->Entries + Map->IndexMask;
			 OldEntry <= FinalEntry; ++OldEntry)
		{
			if (!OldEntry->Open)
			{
				continue;
			}

			// TODO: If hash collisions are slow, we could have a secondary stronger hash here that
			// is used on the second iteration followed by +1 for the subsequent iterations.
			for (size_t Index = FunctionPtrHash(OldEntry->Open, NewCapacity2n);; Index = (Index + 1) & NewMap->IndexMask)
			{
				autortfm_open_to_closed_mapping* NewEntry = NewMap->Entries + Index;
				if (!NewEntry->Open)
				{
					*NewEntry = *OldEntry;
					break;
				}
			}
		}
		NewMap->EntryCount = Map->EntryCount;
		free(Map);
		Map = NewMap;
	}

	// Adds a new entry to the map.
	// The map must have double the capacity of the number of entries before calling.
	void Add(void* Open, void* Closed)
	{
		AUTORTFM_ASSERT(Map->EntryCount + 1 <= (1 << Map->Capacity2n));

		for (size_t Index = FunctionPtrHash(Open, Map->Capacity2n);; Index = (Index + 1) & Map->IndexMask)
		{
			autortfm_open_to_closed_mapping* Mapping = Map->Entries + Index;

			if (!Mapping->Open)
			{
				Mapping->Open = Open;
				Mapping->Closed = Closed;
				Map->EntryCount++;
				return;
			}

			if (Mapping->Open == Open)
			{
				Mapping->Closed = Closed;
				return;
			}
		}
	}

	// Looks up the closed function from the open function pointer.
	// Returns nullptr if the mapping is not found.
	void* Lookup(void* OpenFn)
	{
		for (size_t Index = FunctionPtrHash(OpenFn, Map->Capacity2n);; Index = (Index + 1) & Map->IndexMask)
		{
			autortfm_open_to_closed_mapping* Entry = Map->Entries + Index;

			if (Entry->Open == OpenFn)
			{
				return Entry->Closed;
			}

			if (!Entry->Open)
			{
				return nullptr;
			}
		}
	}

	void DumpStats()
	{
		size_t MapSlots = 1llu << Map->Capacity2n;
		AUTORTFM_LOG("Function Map Stats");
		AUTORTFM_LOG("==================");
		AUTORTFM_LOG("Occupancy: %zu entries in %zu slots (load factor %g%%)", Map->EntryCount, MapSlots,
			static_cast<double>(Map->EntryCount * 1000llu / MapSlots) / 10.0);

		std::map<uintptr_t, size_t> CollisionMap;  // <probes required to find the function, number of functions>
		for (const autortfm_open_to_closed_mapping *Entry = Map->Entries, *FinalEntry = Map->Entries + Map->IndexMask;
			 Entry <= FinalEntry; ++Entry)
		{
			if (Entry->Open)
			{
				uintptr_t ActualIndex = std::distance<const autortfm_open_to_closed_mapping*>(Map->Entries, Entry);
				uintptr_t IdealIndex = FunctionPtrHash(Entry->Open, Map->Capacity2n) & Map->IndexMask;
				// We need to account for wraparound in this calculation.
				uintptr_t Delta = (IdealIndex <= ActualIndex) ? ActualIndex - IdealIndex : ActualIndex + MapSlots - IdealIndex;

				CollisionMap[Delta] += 1;
			}
		}

		for (uintptr_t NumProbes = 0, HighestProbes = CollisionMap.rbegin()->first; NumProbes <= HighestProbes; ++NumProbes)
		{
			AUTORTFM_LOG("%2" PRIuPTR " probes: %zu functions", NumProbes, CollisionMap[NumProbes]);
		}
	}

private:
	FFunctionMap(const FFunctionMap&) = delete;
	FFunctionMap& operator=(const FFunctionMap&) = delete;
	FFunctionMap(FFunctionMap&&) = delete;
	FFunctionMap& operator=(FFunctionMap&&) = delete;

	FFunctionMap(FHashMap*& Map, std::mutex& Mutex) : Map{Map}, Mutex{Mutex}
	{
		Mutex.lock();
	}

	inline static uintptr_t FunctionPtrHash(void* FunctionPtr, size_t HashTableSize2n)
	{
		// Use a Fibonacci hash to fold our size_t value into a hash-appropriate value.
		// https://probablydance.com/2018/06/16/fibonacci-hashing-the-optimization-that-the-world-forgot-or-a-better-alternative-to-integer-modulo/
		uintptr_t HashBits = reinterpret_cast<uintptr_t>(FunctionPtr);
		// Apply Fibonacci product and preserve the highest bits.
		return (HashBits * 11400714819323198485llu) >> (NumBitsInSizeT - HashTableSize2n);
	}

	FHashMap*& Map;
	std::mutex& Mutex;
};

// Attempts to lookup the "true" function from a dynamically linked import function thunk.
// Returns the pointer to the true function, or nullptr if the function cannot
// be resolved (or is not an import thunk).
inline void* FollowRelocation(void* Function)
{
#if AUTORTFM_PLATFORM_WINDOWS
#if defined(_x86_64) || defined(__x86_64__) || defined(_M_X64) || defined(_M_AMD64)
	// Note: Windows has multiple ways to perform relocations:
	// https://learn.microsoft.com/en-us/windows/win32/debug/pe-format#base-relocation-types
	// This handles the only relocation mode currently observed, but may need to
	// support more modes in the future.
	std::byte* Ptr = reinterpret_cast<std::byte*>(Function);
	uint16_t U16;
	memcpy(&U16, Ptr, 2);
	if (U16 == 0x25ff)  // jmp qword ptr [rip+<relative-address>]
	{
		uint32_t RelativeAddress;
		memcpy(&RelativeAddress, Ptr + 2, 4);
		Ptr += RelativeAddress + 6;  // apply relative offset
		memcpy(&Ptr, Ptr, 8);        // load pointer
		return Ptr;
	}
#endif  // is-x64
#endif  // AUTORTFM_PLATFORM_WINDOWS
	return nullptr;
}

AUTORTFM_INTERNAL static void* FunctionMapReportError(void* Open, const char* Where)
{
#if AUTORTFM_PLATFORM_WINDOWS
	if (Open == &InitializeCriticalSection || Open == &InitializeCriticalSectionAndSpinCount || Open == &SetCriticalSectionSpinCount
		|| Open == &TryEnterCriticalSection || Open == &EnterCriticalSection || Open == &LeaveCriticalSection
		|| Open == &DeleteCriticalSection)
	{
		ForTheRuntime::ReportAutoRTFMHazard(ForTheRuntime::EHazardType::CriticalSection);
		return nullptr;
	}
	else if (Open == &TryAcquireSRWLockExclusive || Open == &AcquireSRWLockExclusive || Open == &ReleaseSRWLockExclusive
			 || Open == &TryAcquireSRWLockShared || Open == &AcquireSRWLockShared || Open == &ReleaseSRWLockShared
			 || Open == &TryAcquireSRWLockExclusive)
	{
		ForTheRuntime::ReportAutoRTFMHazard(ForTheRuntime::EHazardType::RWLock);
		return nullptr;
	}
#elif AUTORTFM_PLATFORM_LINUX
	if (Open == &pthread_mutexattr_init || Open == &pthread_mutexattr_settype || Open == &pthread_mutex_init
		|| Open == &pthread_mutexattr_destroy || Open == &pthread_mutex_destroy || Open == &pthread_mutex_trylock
		|| Open == &pthread_mutex_lock || Open == &pthread_mutex_unlock)
	{
		ForTheRuntime::ReportAutoRTFMHazard(ForTheRuntime::EHazardType::CriticalSection);
		return nullptr;
	}
	else if (Open == &pthread_rwlock_init || Open == &pthread_rwlock_destroy || Open == &pthread_rwlock_trywrlock
			 || Open == &pthread_rwlock_wrlock || Open == &pthread_rwlock_unlock || Open == &pthread_rwlock_tryrdlock
			 || Open == &pthread_rwlock_rdlock)
	{
		ForTheRuntime::ReportAutoRTFMHazard(ForTheRuntime::EHazardType::RWLock);
		return nullptr;
	}
#endif

	// This is gross - but if we are allowed to recover from missing functions (the eventual mode
	// we intend to ship AutoRTFM in to users), then we will do an abort in the report error calls
	// below. Which would mean we wouldn't free the allocation backing the string below! Get around
	// it by putting it in an explicit heap allocation, and (ab)use our existing on-abort handling
	// system to free the memory.
	std::string* const FunctionDescription = new std::string(GetFunctionDescription(Open));

	FContext::Get()->GetCurrentTransaction()->DeferUntilAbort([FunctionDescription] { delete FunctionDescription; });

	if (Where)
	{
		AUTORTFM_REPORT_ERROR("Could not find function %p '%s' where '%s'.", Open, FunctionDescription->c_str(), Where);
	}
	else
	{
		AUTORTFM_REPORT_ERROR("Could not find function %p '%s'.", Open, FunctionDescription->c_str());
	}
	return nullptr;
}

}  // anonymous namespace

void FunctionMapDumpStats()
{
	FFunctionMap::Get().DumpStats();
}

void FunctionMapAdd(autortfm_open_to_closed_table* Tables)
{
	// Count the total number of new mappings before taking the lock
	size_t NewMappingCount = 0;
	for (const autortfm_open_to_closed_table* Table = Tables; Table; Table = Table->Next)
	{
		for (const autortfm_open_to_closed_mapping* Mapping = Table->Mappings; Mapping->Open; Mapping++)
		{
			NewMappingCount++;
		}
	}

	FFunctionMap Map = FFunctionMap::Get();
	Map.Reserve(NewMappingCount + Map.Count());

	for (const autortfm_open_to_closed_table* Table = Tables; Table; Table = Table->Next)
	{
		for (const autortfm_open_to_closed_mapping* Mapping = Table->Mappings; Mapping->Open; Mapping++)
		{
			AUTORTFM_VERBOSE("Registering open %p -> %p", Mapping->Open, Mapping->Closed);

			Map.Add(Mapping->Open, Mapping->Closed);

			if (void* OpenRelocated = FollowRelocation(Mapping->Open))
			{
				Map.Add(OpenRelocated, Mapping->Closed);
			}
		}
	}
}

void* FunctionMapLookupExhaustive(void* OpenFn, const char* Where)
{
	// Use an explicit scope for Map as FunctionMapReportError() may unwind the
	// stack without first calling the destructor of Map, leaving the
	// FFunctionMap locked.
	{
		FFunctionMap Map = FFunctionMap::Get();

		if (void* ClosedFn = Map.Lookup(OpenFn); AUTORTFM_LIKELY(ClosedFn))
		{
			return ClosedFn;
		}

		if (void* RelocatedOpenFn = FollowRelocation(OpenFn); AUTORTFM_LIKELY(RelocatedOpenFn))
		{
			if (void* ClosedFn = FunctionMapLookupUsingMagicPrefix(RelocatedOpenFn); AUTORTFM_LIKELY(ClosedFn))
			{
				return ClosedFn;
			}
			if (void* ClosedFn = Map.Lookup(RelocatedOpenFn); AUTORTFM_LIKELY(ClosedFn))
			{
				return ClosedFn;
			}
		}
	}

	AUTORTFM_MUST_TAIL return FunctionMapReportError(OpenFn, Where);
}

}  // namespace AutoRTFM

#endif  // defined(__AUTORTFM) && __AUTORTFM
