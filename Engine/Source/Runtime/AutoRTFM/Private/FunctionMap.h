// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Utils.h"

namespace AutoRTFM
{

// Dumps statistics about the function map to LOG_INFO.
AUTORTFM_INTERNAL void FunctionMapDumpStats();

// Adds all the open to closed function mappings from the linked list of tables to the function map.
AUTORTFM_INTERNAL void FunctionMapAdd(autortfm_open_to_closed_table* Tables);

// Looks up the closed function from the open function using the function map, following dynamic
// library thunk redirections.
AUTORTFM_INTERNAL void* FunctionMapLookupExhaustive(void* OpenFn, const char* Where);

template <typename ReturnType, typename... ParameterTypes>
AUTORTFM_INTERNAL auto FunctionMapLookupExhaustive(ReturnType (*Function)(ParameterTypes...), const char* Where)
	-> ReturnType (*)(ParameterTypes...)
{
	return reinterpret_cast<ReturnType (*)(ParameterTypes...)>(
		FunctionMapLookupExhaustive(reinterpret_cast<void*>(Function), Where));
}

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
