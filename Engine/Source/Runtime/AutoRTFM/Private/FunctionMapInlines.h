// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM/Constants.h"
#include "AutoRTFM/Defines.h"
#include "FunctionMap.h"
#include "Utils.h"

namespace AutoRTFM
{

AUTORTFM_INTERNAL UE_AUTORTFM_FORCEINLINE void* FunctionMapLookupUsingMagicPrefix(void* OpenFn)
{
	constexpr uint64_t UbsanMagic = 0xc105'cafe;

	// We use prefix data in our custom LLVM pass to stuff some data just
	// before the address of all open function pointers (that we have
	// definitions for!). We verify the special Magic Prefix constant in the
	// top 16-bits of the function pointer address as a magic constant check
	// to give us a much higher confidence that there is actually a closed
	// variant pointer residing 8-bytes before our function address.
	uint64_t PrefixData;
	char* CharOpenFn = reinterpret_cast<char*>(OpenFn);
	memcpy(&PrefixData, CharOpenFn - sizeof(uint64_t), sizeof(uint64_t));
#if PLATFORM_LINUX
	constexpr uint64_t Mask = 0xffff'0000'0000'0000;
	if (AUTORTFM_LIKELY(Constants::PosOffsetMagicPrefix == (PrefixData & Mask)))
	{
		return reinterpret_cast<void*>(CharOpenFn + (PrefixData & ~Mask));
	}
	else if (AUTORTFM_LIKELY((Constants::NegOffsetMagicPrefix) == (PrefixData & Mask)))
	{
		// when NegOffsetMagicPrefix is matched, that means the offset was negative and we need to sign extend
		return reinterpret_cast<void*>(CharOpenFn + (int64_t)(PrefixData | Mask));
	}
	// UBSAN adds a type hash prefix to the function as a "prologue" that
	// ends preceding our Magic Prefix. They use 0xc105cafe in the
	// lower 32-bits to distinguish the 64-bit word containing their type
	// hash. If we see it, check the preceding 64-bit word for our prefix.
	else if (AUTORTFM_UNLIKELY(UbsanMagic == (PrefixData & 0x0000'0000'ffff'ffff)))
	{
		memcpy(&PrefixData, reinterpret_cast<char*>(OpenFn) - sizeof(uint64_t) * 2, sizeof(uint64_t));
		if (AUTORTFM_LIKELY(Constants::PosOffsetMagicPrefix == (PrefixData & Mask)))
		{
			return reinterpret_cast<void*>(CharOpenFn + (PrefixData & ~Mask));
		}
		else if (AUTORTFM_LIKELY((Constants::NegOffsetMagicPrefix) == (PrefixData & Mask)))
		{
			// when NegOffsetMagicPrefix is matched, that means the offset was negative and we need to sign extend
			return reinterpret_cast<void*>(CharOpenFn + (int64_t)(PrefixData | Mask));
		}
	}
#else
	memcpy(&PrefixData, reinterpret_cast<char*>(OpenFn) - sizeof(uint64_t), sizeof(uint64_t));
	if (AUTORTFM_LIKELY(Constants::MagicPrefix == (PrefixData & 0xffff'0000'0000'0000)))
	{
		return reinterpret_cast<void*>(PrefixData & 0x0000'ffff'ffff'ffff);
	}
	// UBSAN adds a type hash prefix to the function as a "prologue" that
	// ends preceding our Magic Prefix. They use 0xc105cafe in the
	// lower 32-bits to distinguish the 64-bit word containing their type
	// hash. If we see it, check the preceding 64-bit word for our prefix.
	else if (AUTORTFM_UNLIKELY(UbsanMagic == (PrefixData & 0x0000'0000'ffff'ffff)))
	{
		memcpy(&PrefixData, reinterpret_cast<char*>(OpenFn) - sizeof(uint64_t) * 2, sizeof(uint64_t));
		if (AUTORTFM_LIKELY(Constants::MagicPrefix == (PrefixData & 0xffff'0000'0000'0000)))
		{
			return reinterpret_cast<void*>(PrefixData & 0x0000'ffff'ffff'ffff);
		}
	}
#endif

	return nullptr;
}

AUTORTFM_INTERNAL inline void* FunctionMapLookup(void* OpenFn, const char* Where)
{
	if (void* ClosedFn = FunctionMapLookupUsingMagicPrefix(OpenFn); AUTORTFM_LIKELY(ClosedFn))
	{
		return ClosedFn;
	}

	AUTORTFM_MUST_TAIL return FunctionMapLookupExhaustive(OpenFn, Where);
}

template <typename TReturnType, typename... TParameterTypes>
AUTORTFM_INTERNAL auto FunctionMapLookup(TReturnType (*OpenFn)(TParameterTypes...), const char* Where)
	-> TReturnType (*)(TParameterTypes...)
{
	return reinterpret_cast<TReturnType (*)(TParameterTypes...)>(FunctionMapLookup(reinterpret_cast<void*>(OpenFn), Where));
}

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
