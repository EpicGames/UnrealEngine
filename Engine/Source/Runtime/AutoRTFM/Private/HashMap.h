// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Allocator.h"
#include "StlAllocator.h"

#include <functional>
#include <type_traits>
#include <unordered_map>

namespace AutoRTFM
{

// A simple structure holding a templated key and value pair.
template <typename KeyType, typename ValueType>
struct TKeyAndValue
{
	KeyType Key;
	ValueType Value;
};

// HashMap is an unordered hashmap.
// TODO(SOL-7652): This currently wraps a std::unordered_map. Reimplement to
// improve performance and to avoid platform-specific variations in behavior.
template <typename KeyType, typename ValueType, typename Hash = std::hash<KeyType>, typename Equal = std::equal_to<KeyType>,
	Allocator Allocator = FAllocator>
class AUTORTFM_DISABLE THashMap
{
	using FInnerMap =
		std::unordered_map<KeyType, ValueType, Hash, Equal, StlAllocator<std::pair<const KeyType, ValueType>, Allocator>>;

	template <bool bConst>
	class TIterator
	{
		using FInnerIterator = std::conditional_t<bConst, typename FInnerMap::const_iterator, typename FInnerMap::iterator>;
		using FKeyAndValue =
			std::conditional_t<bConst, const TKeyAndValue<const KeyType, ValueType>, TKeyAndValue<const KeyType, ValueType>>;

	public:
		// Constructor wrapping inner-iterator
		TIterator(FInnerIterator InnerIterator) : InnerIterator{InnerIterator} {}
		// Copy constructor
		TIterator(const TIterator&) = default;
		// Move constructor
		TIterator(TIterator&&) = default;
		// Copy assignment operator
		TIterator& operator=(const TIterator& Other) = default;
		// Move assignment operator
		TIterator& operator=(TIterator&& Other) = default;

		FKeyAndValue& operator*() const
		{
			// Super hacky: Reinterpret a std::pair to a TKeyAndValue so
			// that THashMap's iterator has the interface of a TMap instead of a
			// std::unordered_map.
			// TODO(SOL-7652): Remove this filth when THashMap is reimplemented.
			using FPair = std::conditional_t<bConst, const std::pair<const KeyType, ValueType>, std::pair<const KeyType, ValueType>>;
			FPair* Pair = &*InnerIterator;
			FKeyAndValue* KeyAndValue = reinterpret_cast<FKeyAndValue*>(Pair);
			static_assert(std::is_same_v<decltype(Pair->first), decltype(KeyAndValue->Key)>);
			static_assert(std::is_same_v<decltype(Pair->second), decltype(KeyAndValue->Value)>);
			static_assert(offsetof(FPair, first) == offsetof(FKeyAndValue, Key));
			static_assert(offsetof(FPair, second) == offsetof(FKeyAndValue, Value));
			static_assert(sizeof(FPair) == sizeof(FKeyAndValue));
			static_assert(alignof(FPair) == alignof(FKeyAndValue));
			return *KeyAndValue;
		}

		TIterator& operator++()
		{
			++InnerIterator;
			return *this;
		}

		bool operator==(const TIterator& Other) const
		{
			return InnerIterator == Other.InnerIterator;
		}

		bool operator!=(const TIterator& Other) const
		{
			return InnerIterator != Other.InnerIterator;
		}

	private:
		FInnerIterator InnerIterator;
	};

public:
	using Key = KeyType;
	using Value = ValueType;
	using Iterator = TIterator</* bConst */ false>;
	using ConstIterator = TIterator</* bConst */ true>;

	// Constructor
	THashMap() = default;
	// Destructor
	~THashMap() = default;
	// Copy constructor
	THashMap(const THashMap& Other) = default;
	// Move constructor
	THashMap(THashMap&& Other) : Map{std::move(Other.Map)}
	{
		if (this != &Other)
		{
			Other.Map.clear();  // Ensure Other is cleared.
		}
	}
	// Copy assignment operator
	THashMap& operator=(const THashMap& Other) = default;
	// Move assignment operator
	THashMap& operator=(THashMap&& Other)
	{
		if (this != &Other)
		{
			Map = std::move(Other.Map);
			Other.Map.clear();  // Ensure Other is cleared.
		}
		return *this;
	}

	// Set the value associated with a key, replacing any existing entry with
	// the given key.
	template <typename K, typename V>
	void Add(K&& Key, V&& Value)
	{
		Map[std::forward<K>(Key)] = std::forward<V>(Value);
	}

	// Looks up the value with the given key, returning a pointer to the value
	// if found or nullptr if not found.
	// Warning: The returned pointer will become invalid if the hash map is
	// modified.
	template <typename K>
	ValueType* Find(K&& Key)
	{
		typename FInnerMap::iterator It = Map.find(std::forward<K>(Key));
		return It == Map.end() ? nullptr : &It->second;
	}

	// Looks up the value with the given key, returning a reference to the
	// existing value if found or a reference to a newly added, zero-initialized
	// value if not found.
	// Warning: The returned reference will become invalid if the hash map is
	// modified.
	template <typename K>
	ValueType& FindOrAdd(K&& Key)
	{
		return Map[std::forward<K>(Key)];
	}

	// Removes the entry with the given key. This is a no-op if the hash map
	// does not contain an entry with the given key.
	void Remove(const KeyType& Key)
	{
		Map.erase(Key);
	}

	// Returns true if the hash map contains an entry with the given key.
	bool Contains(const KeyType& Key)
	{
		return Map.count(Key) != 0;
	}

	// Removes all entries from the hash map, freeing all allocations.
	void Empty()
	{
		Map.clear();
	}

	// Removes all entries from the hash map.
	// TODO(SOL-7652): Preserve any heap allocations made by the hash map.
	void Reset()
	{
		Map.clear();
	}

	// Returns the number of entries held by the hash map.
	size_t Num() const
	{
		return Map.size();
	}

	// Returns true if the hash map holds no entries.
	bool IsEmpty() const
	{
		return Map.empty();
	}

	// Iterator methods
	ConstIterator begin() const
	{
		return ConstIterator{Map.begin()};
	}
	ConstIterator end() const
	{
		return ConstIterator{Map.end()};
	}
	Iterator begin()
	{
		return Iterator{Map.begin()};
	}
	Iterator end()
	{
		return Iterator{Map.end()};
	}

private:
	FInnerMap Map;
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
