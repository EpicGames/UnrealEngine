// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Allocator.h"
#include "Utils.h"

#include <cstdint>

namespace AutoRTFM
{
struct FHitSetEntry
{
	static constexpr size_t AddressBits = 48;
	static constexpr size_t SizeBits = 12;
	static constexpr size_t FlagsBits = 4;

	// The address of the write.
	uintptr_t Address : AddressBits;
	// Size of the write.
	uintptr_t Size : SizeBits;
	// Flags applied to the write.
	uintptr_t Flags : FlagsBits;

	UE_AUTORTFM_FORCEINLINE uintptr_t Payload() const
	{
		// Note: Clang optimizes this away to a reinterpret cast.
		uintptr_t Result;
		memcpy(&Result, this, sizeof(Result));
		return Result;
	}
};

static_assert(sizeof(FHitSetEntry) == sizeof(uintptr_t));

class AUTORTFM_DISABLE FHitSet final
{
public:
	// Maximum size in bytes for a hit-set write record.
	// The cutoff here is arbitrarily any number less than UINT16_MAX, but its a
	// weigh up what a good size is. Because the hitset doesn't detect when you
	// are trying to write to a subregion of a previous hit (like memset something,
	// then write to an individual element), we've got to balance the cost of
	// recording meaningless hits, against the potential to hit again.
	static constexpr size_t MaxSize = 16;

	// TODO: Revisit a good max capacity for the hitset.
	static constexpr uint64_t MaxCapacity = static_cast<uint64_t>(128 * 1024 * 1024) / sizeof(FHitSetEntry);

private:
	static constexpr uint8_t LinearProbeDepth = 32;
	static constexpr uint8_t LogInitialCapacity = 6;
	static constexpr uint64_t InitialCapacity = static_cast<uint64_t>(1) << LogInitialCapacity;

	static_assert(LinearProbeDepth <= InitialCapacity);

public:
	~FHitSet()
	{
		Reset();
	}

	enum class EInsertResult : uint8_t
	{
		Exists,
		Inserted,
		NotInserted
	};

	// Insert something in the HitSet, returning whether the key already
	// exists, it was inserted, or not.
	UE_AUTORTFM_FORCEINLINE EInsertResult FindOrTryInsertNoResize(FHitSetEntry Entry)
	{
		return InsertNoResize(Entry.Payload());
	}

	// Insert something in the HitSet, returning true if we found the key.
	// If false is returned, we could have inserted or not - depending on
	// whether the HitSet has reached its max capacity.
	UE_AUTORTFM_FORCEINLINE bool FindOrTryInsert(FHitSetEntry Entry)
	{
		while (true)
		{
			switch (InsertNoResize(Entry.Payload()))
			{
				default:
					InternalUnreachable();
				case EInsertResult::Inserted:
					return false;
				case EInsertResult::Exists:
					return true;
				case EInsertResult::NotInserted:
					if (!Resize())
					{
						return false;
					}
					continue;
			}
		}
	}

	// Returns true if the HitSet contains the given entry, without mutation.
	UE_AUTORTFM_FORCEINLINE bool Contains(FHitSetEntry Entry) const
	{
		uintptr_t Raw = Entry.Payload();
		const uintptr_t Hash = FibonacciHash(Raw);

		// The first index will always be in the range of capacity because we
		// are using a fibonacci hash.
		{
			const uintptr_t Value = Payload[Hash];
			if (0 == Value)
			{
				return false;
			}
			if (Raw == Value)
			{
				return true;
			}
			// Otherwise we need to do a linear probe...
		}

		const uintptr_t Mask = Capacity() - 1;

		// Clang goes way over the top with loop unrolling, and makes this code noticeably
		// slower as a result. So we disable it!
#ifdef __clang__
#pragma clang loop unroll(disable)
#endif
		for (uint8_t D = 1; D < LinearProbeDepth; D++)
		{
			// Capacity is always a power of 2, so we can just mask out the bits.
			const uintptr_t I = (Hash + D) & Mask;

			const uintptr_t Value = Payload[I];
			if (0 == Value)
			{
				return false;
			}
			if (Raw == Value)
			{
				return true;
			}

			// Otherwise we need to keep going with our linear probe...
		}

		return false;
	}

	UE_AUTORTFM_FORCEINLINE bool IsEmpty() const
	{
		return 0 == Count;
	}

	// Clear out the data stored in the set, but does not reduce the capacity.
	void Reset()
	{
		if (Count)
		{
			if (Payload != SmallPayload)
			{
				size_t const PayloadSize = Capacity() * sizeof(uintptr_t);
				FAllocator::Free(Payload, PayloadSize);
				Payload = SmallPayload;
			}
			memset(SmallPayload, 0, sizeof(SmallPayload));
			SixtyFourMinusLogCapacity = 64 - LogInitialCapacity;
			Count = 0;
		}
	}

	uint64_t GetCapacity() const
	{
		return Capacity();
	}

	uint64_t GetCount() const
	{
		return Count;
	}

	struct FIterator
	{
		FIterator(uintptr_t const* Ptr, size_t EntriesRemaining) : Ptr{Ptr}, EntriesRemaining{EntriesRemaining}
		{
			SkipEmpty();
		}

		FHitSetEntry operator*() const
		{
			FHitSetEntry Out;
			memcpy(&Out, Ptr, sizeof(Out));
			return Out;
		}

		FIterator& operator++()
		{
			if (EntriesRemaining)
			{
				EntriesRemaining--;
				Ptr++;
				SkipEmpty();
			}
			return *this;
		}

		bool operator==(FIterator Other) const
		{
			return Other.Ptr == Ptr;
		}

		bool operator!=(FIterator Other) const
		{
			return Other.Ptr != Ptr;
		}

	private:
		void SkipEmpty()
		{
			while (EntriesRemaining > 0 && *Ptr == 0)
			{
				EntriesRemaining--;
				Ptr++;
			}
		}

		const uintptr_t* Ptr;
		size_t EntriesRemaining;
	};

	FIterator begin() const
	{
		return FIterator(Payload, Capacity());
	}
	FIterator end() const
	{
		return FIterator(Payload + Capacity(), 0);
	}

private:
	uintptr_t SmallPayload[InitialCapacity]{};
	uintptr_t* Payload = SmallPayload;
	uint64_t Count = 0;
	uint32_t SixtyFourMinusLogCapacity = 64 - LogInitialCapacity;

	UE_AUTORTFM_FORCEINLINE uint64_t Capacity() const
	{
		UE_AUTORTFM_ASSUME(SixtyFourMinusLogCapacity < 64);
		return static_cast<uint64_t>(1) << (64 - SixtyFourMinusLogCapacity);
	}

	UE_AUTORTFM_FORCEINLINE void IncreaseCapacity()
	{
		// It seems odd - but subtracting 1 here is totally intentional
		// because of how we store our capacity as (64 - log2(Capacity)).
		SixtyFourMinusLogCapacity -= 1;

		// Check that we haven't overflowed our capacity!
		AUTORTFM_ASSERT(0 != SixtyFourMinusLogCapacity);
	}

	bool Resize()
	{
		const uint64_t OldCapacity = Capacity();

		if (OldCapacity >= MaxCapacity)
		{
			return false;
		}

		uintptr_t* const OldPayload = Payload;
		const uint64_t OldCount = Count;
		size_t OldPayloadSize = Capacity() * sizeof(uintptr_t);

		while (true)
		{
			bool bNeedAnotherResize = false;

			Count = 0;

			IncreaseCapacity();

			size_t const NewPayloadSize = Capacity() * sizeof(uintptr_t);
			Payload = static_cast<uintptr_t*>(FAllocator::AllocateZeroed(NewPayloadSize, alignof(uintptr_t)));
			AUTORTFM_ASSERT(nullptr != Payload);

			// Now we need to rehash and reinsert all the items.
			for (size_t I = 0; I < OldCapacity; I++)
			{
				// Skip empty locations.
				if (0 == OldPayload[I])
				{
					continue;
				}

				const EInsertResult Result = InsertNoResize(OldPayload[I]);

				if (EInsertResult::NotInserted == Result)
				{
					bNeedAnotherResize = true;
					break;
				}
				else
				{
					AUTORTFM_ASSERT(EInsertResult::Inserted == Result);
				}
			}

			if (bNeedAnotherResize)
			{
				FAllocator::Free(Payload, NewPayloadSize);
				continue;
			}

			break;
		}

		AUTORTFM_ASSERT(OldCount == Count);

		if (OldPayload != SmallPayload)
		{
			FAllocator::Free(OldPayload, OldPayloadSize);
		}

		return true;
	}

	UE_AUTORTFM_FORCEINLINE uintptr_t FibonacciHash(const uintptr_t Hashee) const
	{
		constexpr uintptr_t Fibonacci = 0x9E3779B97F4A7C15;
		return (Hashee * Fibonacci) >> SixtyFourMinusLogCapacity;
	}

	UE_AUTORTFM_FORCEINLINE EInsertResult TryInsertAtIndex(const uintptr_t Raw, const uintptr_t I)
	{
		// We have a free location in the set.
		if (0 == Payload[I])
		{
			Payload[I] = Raw;
			Count++;
			return EInsertResult::Inserted;
		}

		// We're already in the set.
		if (Raw == Payload[I])
		{
			return EInsertResult::Exists;
		}

		return EInsertResult::NotInserted;
	}

	// Insert something in the HitSet, returning true if the insert
	// succeeded (EG. the key was not already in the set).
	UE_AUTORTFM_FORCEINLINE EInsertResult InsertNoResize(const uintptr_t Raw)
	{
		AUTORTFM_ASSERT_DEBUG(Raw != 0);

		const uintptr_t Hash = FibonacciHash(Raw);

		// The first index will always be in the range of capacity because we
		// are using a fibonacci hash. So just check if we can insert here.
		{
			const uintptr_t I = Hash;

			const EInsertResult R = TryInsertAtIndex(Raw, I);

			if (EInsertResult::NotInserted != R)
			{
				return R;
			}

			// Otherwise we need to do a linear probe...
		}

		const uintptr_t Mask = Capacity() - 1;

		// Clang goes way over the top with loop unrolling, and makes this code noticeably
		// slower as a result. So we disable it!
#ifdef __clang__
#pragma clang loop unroll(disable)
#endif
		for (uint8_t D = 1; D < LinearProbeDepth; D++)
		{
			// Capacity is always a power of 2, so we can just mask out the bits.
			const uintptr_t I = (Hash + D) & Mask;

			const EInsertResult R = TryInsertAtIndex(Raw, I);

			if (EInsertResult::NotInserted != R)
			{
				return R;
			}

			// Otherwise we need to keep going with our linear probe...
		}

		return EInsertResult::NotInserted;
	}
};
}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
