// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include <cstddef>
#include <cstdint>

namespace AutoRTFM
{

// A half-open address interval [Start, End).
struct FInterval final
{
	// Default constructor - leaves Start and End uninitialized.
	FInterval() = default;

	// Constructs an interval with explicit start and end addresses.
	constexpr FInterval(uintptr_t Start, uintptr_t End) : Start{Start}, End{End} {}

	// Constructs an interval from a base address and byte size.
	FInterval(const void* const Address, const size_t Size)
		: Start(reinterpret_cast<uintptr_t>(Address))
		, End(reinterpret_cast<uintptr_t>(Address) + Size)
	{
	}

	// Returns the number of bytes in the interval.
	size_t Size() const
	{
		return End - Start;
	}

	// Orders intervals by start address, for use in sorted containers.
	bool operator<(const FInterval& Other) const
	{
		return Start < Other.Start;
	}

	uintptr_t Start;
	uintptr_t End;
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
