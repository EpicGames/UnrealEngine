// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Allocator.h"
#include "ContainerValidation.h"
#include "Interval.h"
#include "Stack.h"
#include "Utils.h"

#include <cstring>

namespace AutoRTFM
{

// TIntervalTree tracks a set of non-overlapping memory intervals, each associated with a
// data value of type DataTypeIn, using a red-black tree. Intervals are stored as [Start, End).
// On insert, the new interval takes ownership of its address range:
//   - Overlapping or adjacent intervals with the same data are merged.
//   - Overlapping intervals with different data are trimmed, split, or deleted.
template <typename DataTypeIn, Allocator Allocator = FAllocator>
requires std::equality_comparable<DataTypeIn>
struct AUTORTFM_INTERNAL TIntervalTree final
{
private:
	using NodeIndex = uint32_t;
	using DataType = DataTypeIn;
	static constexpr NodeIndex InvalidNode = UINT32_MAX;

public:
	TIntervalTree() = default;

	TIntervalTree(const TIntervalTree&) = delete;
	TIntervalTree& operator=(const TIntervalTree&) = delete;

	// Inserts the interval [Address, Address+Size) with the given data into the tree.
	// The new interval takes ownership of its address range:
	//   - Overlapping or adjacent same-data intervals are merged into one.
	//   - Overlapping different-data intervals are trimmed, split, or deleted to make room.
	void Insert(const void* const Address, const size_t Size, DataType&& Data)
	{
		Insert(FInterval{Address, Size}, std::move(Data));
	}

	// Inserts the interval [Address, Address+Size) with the given data into the tree.
	// The new interval takes ownership of its address range:
	//   - Overlapping or adjacent same-data intervals are merged into one.
	//   - Overlapping different-data intervals are trimmed, split, or deleted to make room.
	void Insert(const void* const Address, const size_t Size, const DataType& Data)
	{
		Insert(FInterval{Address, Size}, Data);
	}

	// Returns true if no intervals are stored.
	bool IsEmpty() const
	{
		return InvalidNode == Root;
	}

	// Removes all stored intervals.
	void Reset()
	{
		Root = InvalidNode;
		Nodes.Reset();
	}

	// Inserts all intervals from Other into this tree, applying the same ownership
	// semantics as Insert: overlapping/adjacent same-data intervals merge, and
	// overlapping different-data intervals are trimmed, split, or deleted.
	void Merge(const TIntervalTree& Other)
	{
		if (Other.IsEmpty())
		{
			return;
		}

		TStack<NodeIndex, 32, Allocator, EContainerValidation::Disabled> ToProcess;

		ToProcess.Push(Other.Root);

		do
		{
			const NodeIndex Current = ToProcess.Back();
			ToProcess.Pop();

			Insert(Other.Nodes.Interval(Current), Other.Nodes.Data(Current));

			const NodeIndex Left = Other.Nodes.Left(Current);
			const NodeIndex Right = Other.Nodes.Right(Current);

			if (InvalidNode != Left)
			{
				ToProcess.Push(Left);
			}

			if (InvalidNode != Right)
			{
				ToProcess.Push(Right);
			}
		} while (!ToProcess.IsEmpty());
	}

	// A single entry yielded by the iterator: the stored interval and a reference to its data.
	struct FIntervalAndData
	{
		FInterval Interval;
		const DataType& Data;
	};

	// A forward iterator that yields intervals in ascending order of start address.
	struct FIterator
	{
		FIterator(const FIterator&) = default;
		FIterator& operator=(const FIterator&) = default;

		FIntervalAndData operator*() const
		{
			return {.Interval = Tree->Nodes.Interval(Node), .Data = Tree->Nodes.Data(Node)};
		}

		FIterator& operator++()
		{
			Node = Tree->Next(Node);
			return *this;
		}

		bool operator==(const FIterator& Other) const = default;

	private:
		friend struct TIntervalTree;
		FIterator(TIntervalTree const* Tree, NodeIndex Node) : Tree{Tree}, Node{Node} {}
		TIntervalTree const* Tree = nullptr;
		NodeIndex Node = InvalidNode;
	};

	// Returns an iterator to the first interval in the tree
	FIterator begin() const
	{
		return FIterator(this, Root == InvalidNode ? InvalidNode : LeftMostDescendant(Root));
	}

	// Returns an end iterator for the tree
	FIterator end() const
	{
		return FIterator(this, InvalidNode);
	}

	// A lightweight range representing all stored intervals that overlap a query.
	struct FOverlapRange
	{
		FIterator Begin;
		FIterator End;
		FIterator begin() const
		{
			return Begin;
		}
		FIterator end() const
		{
			return End;
		}
	};

	// Returns true if any stored interval overlaps [Address, Address+Size).
	// Faster than Get() when you only need an existence check: single tree traversal,
	// no iterator construction, no end-node lookup.
	UE_AUTORTFM_FORCENOINLINE bool Overlaps(const void* const Address, const size_t Size) const
	{
		if (InvalidNode == Root)
		{
			return false;
		}

		const uintptr_t QueryStart = reinterpret_cast<uintptr_t>(Address);
		const uintptr_t QueryEnd = QueryStart + Size;

		const FInterval* const Intervals = Nodes.Ptrs.Interval;
		const NodeIndex* const Lefts = Nodes.Ptrs.Left;
		const NodeIndex* const Rights = Nodes.Ptrs.Right;

		NodeIndex LowerBoundNode = InvalidNode;
		NodeIndex PredecessorNode = InvalidNode;
		{
			NodeIndex Current = Root;
			while (Current != InvalidNode)
			{
				if (Intervals[Current].Start >= QueryStart)
				{
					LowerBoundNode = Current;
					Current = Lefts[Current];
				}
				else
				{
					PredecessorNode = Current;
					Current = Rights[Current];
				}
			}
		}

		if (AUTORTFM_UNLIKELY(PredecessorNode != InvalidNode && Intervals[PredecessorNode].End > QueryStart))
		{
			return true;
		}

		return AUTORTFM_UNLIKELY(LowerBoundNode != InvalidNode && Intervals[LowerBoundNode].Start < QueryEnd);
	}

	// Returns all stored intervals overlapping [Address, Address+Size), in ascending
	// order of start address.
	UE_AUTORTFM_FORCENOINLINE FOverlapRange Get(const void* const Address, const size_t Size) const
	{
		if (AUTORTFM_UNLIKELY(InvalidNode == Root))
		{
			return {end(), end()};
		}

		const uintptr_t QueryStart = reinterpret_cast<uintptr_t>(Address);
		const uintptr_t QueryEnd = QueryStart + Size;

		const FInterval* const Intervals = Nodes.Ptrs.Interval;
		const NodeIndex* const Lefts = Nodes.Ptrs.Left;
		const NodeIndex* const Rights = Nodes.Ptrs.Right;

		NodeIndex LowerBoundNode = InvalidNode;
		NodeIndex PredecessorNode = InvalidNode;
		{
			NodeIndex Current = Root;
			while (Current != InvalidNode)
			{
				if (Intervals[Current].Start >= QueryStart)
				{
					LowerBoundNode = Current;
					Current = Lefts[Current];
				}
				else
				{
					PredecessorNode = Current;
					Current = Rights[Current];
				}
			}
		}

		NodeIndex BeginNode = LowerBoundNode;
		if (PredecessorNode != InvalidNode && Intervals[PredecessorNode].End > QueryStart)
		{
			BeginNode = PredecessorNode;
		}

		if (BeginNode == InvalidNode || Intervals[BeginNode].Start >= QueryEnd)
		{
			return {end(), end()};
		}

		const NodeIndex EndNode = LowerBound(QueryEnd);
		return {FIterator(this, BeginNode), FIterator(this, EndNode)};
	}

private:
	friend struct FIterator;

	enum EColor : bool
	{
		Red = false,
		Black = true
	};

	using ColorWord = uint64_t;

	struct FNode final
	{
		FInterval Interval;
		NodeIndex Left = InvalidNode;
		NodeIndex Right = InvalidNode;
		NodeIndex Parent = InvalidNode;
		EColor Color;
		DataType Data;
	};

	struct FNodePtrs final
	{
		FInterval* Interval = nullptr;
		NodeIndex* Left = nullptr;
		NodeIndex* Right = nullptr;
		NodeIndex* Parent = nullptr;
		ColorWord* Color = nullptr;
		DataType* Data = nullptr;
	};

	// A list of FNodes, stored as a structure-of-arrays.
	struct FNodes final
	{
		static constexpr size_t NumColorWordBits = sizeof(ColorWord) * 8;
		static constexpr NodeIndex InitialCapacity = 32;

		FNodePtrs Ptrs;
		NodeIndex Count = 0;
		NodeIndex Capacity = 0;

		FNodes() = default;

		~FNodes()
		{
			Reset();
		}

		// Clears the node list to an empty state.
		void Reset()
		{
			for (size_t I = 0; I < Count; I++)
			{
				Ptrs.Data[I].~DataType();
			}

			Ptrs = {};
			Count = 0;
			Capacity = 0;

			Allocator::Free(Memory, MemorySizeInBytes);
			Memory = nullptr;
			MemorySizeInBytes = 0;
		}

		// Pushes a new node to the end of the list.
		// Returns the node index.
		NodeIndex Push(FNode Node)
		{
			if (Count >= Capacity)
			{
				Resize(std::max(Capacity * 2, InitialCapacity));
			}

			Ptrs.Interval[Count] = Node.Interval;
			Ptrs.Left[Count] = Node.Left;
			Ptrs.Right[Count] = Node.Right;
			Ptrs.Parent[Count] = Node.Parent;
			SetColor(Count, Node.Color);
			new (&Ptrs.Data[Count]) DataType(std::move(Node.Data));

			return static_cast<NodeIndex>(Count++);
		}

		FInterval& Interval(NodeIndex Index)
		{
			return Ptrs.Interval[Index];
		}
		const FInterval& Interval(NodeIndex Index) const
		{
			return Ptrs.Interval[Index];
		}
		NodeIndex& Left(NodeIndex Index)
		{
			return Ptrs.Left[Index];
		}
		const NodeIndex& Left(NodeIndex Index) const
		{
			return Ptrs.Left[Index];
		}
		NodeIndex& Right(NodeIndex Index)
		{
			return Ptrs.Right[Index];
		}
		const NodeIndex& Right(NodeIndex Index) const
		{
			return Ptrs.Right[Index];
		}
		NodeIndex& Parent(NodeIndex Index)
		{
			return Ptrs.Parent[Index];
		}
		const NodeIndex& Parent(NodeIndex Index) const
		{
			return Ptrs.Parent[Index];
		}
		DataType& Data(NodeIndex Index)
		{
			return Ptrs.Data[Index];
		}
		const DataType& Data(NodeIndex Index) const
		{
			return Ptrs.Data[Index];
		}

		void SetColor(NodeIndex Index, EColor Color)
		{
			ColorWord const Bit = static_cast<ColorWord>(1) << (Index & (NumColorWordBits - 1));
			ColorWord& Word = Ptrs.Color[Index / NumColorWordBits];
			if (Color == EColor::Black)
			{
				Word |= Bit;
			}
			else
			{
				Word &= ~Bit;
			}
		}

		EColor Color(NodeIndex Index) const
		{
			ColorWord const Bit = static_cast<ColorWord>(1) << (Index & (NumColorWordBits - 1));
			const ColorWord Word = Ptrs.Color[Index / NumColorWordBits];
			return (Word & Bit) ? EColor::Black : EColor::Red;
		}

	private:
		std::byte* Memory = nullptr;
		size_t MemorySizeInBytes = 0;

		FNodes(const FNodes&) = delete;
		FNodes& operator=(const FNodes&) = delete;

		void Resize(NodeIndex NewCapacity)
		{
			size_t TotalSize = 0;
			size_t const IntervalOffset = SubAllocate<FInterval>(TotalSize, NewCapacity);
			size_t const LeftOffset = SubAllocate<NodeIndex>(TotalSize, NewCapacity);
			size_t const RightOffset = SubAllocate<NodeIndex>(TotalSize, NewCapacity);
			size_t const ParentOffset = SubAllocate<NodeIndex>(TotalSize, NewCapacity);
			size_t const ColorOffset = SubAllocate<ColorWord>(TotalSize, NumColorWords(NewCapacity));
			size_t const DataOffset = SubAllocate<DataType>(TotalSize, NewCapacity);
			auto* const NewMemory = reinterpret_cast<std::byte*>(Allocator::Allocate(TotalSize, alignof(FNode)));
			auto* const NewInterval = std::launder(reinterpret_cast<FInterval*>(NewMemory + IntervalOffset));
			auto* const NewLeft = std::launder(reinterpret_cast<NodeIndex*>(NewMemory + LeftOffset));
			auto* const NewRight = std::launder(reinterpret_cast<NodeIndex*>(NewMemory + RightOffset));
			auto* const NewParent = std::launder(reinterpret_cast<NodeIndex*>(NewMemory + ParentOffset));
			auto* const NewColor = std::launder(reinterpret_cast<ColorWord*>(NewMemory + ColorOffset));
			auto* const NewData = std::launder(reinterpret_cast<DataType*>(NewMemory + DataOffset));
			if (size_t const NumToCopy = std::min(Count, NewCapacity))
			{
				memcpy(NewInterval, Ptrs.Interval, sizeof(FInterval) * NumToCopy);
				memcpy(NewLeft, Ptrs.Left, sizeof(NodeIndex) * NumToCopy);
				memcpy(NewRight, Ptrs.Right, sizeof(NodeIndex) * NumToCopy);
				memcpy(NewParent, Ptrs.Parent, sizeof(NodeIndex) * NumToCopy);
				memcpy(NewColor, Ptrs.Color, sizeof(ColorWord) * NumColorWords(NumToCopy));

				for (size_t I = 0; I < NumToCopy; I++)
				{
					new (&NewData[I]) DataType(std::move(Ptrs.Data[I]));
					Ptrs.Data[I].~DataType();
				}
			}

			Allocator::Free(Memory, MemorySizeInBytes);

			Ptrs.Interval = NewInterval;
			Ptrs.Left = NewLeft;
			Ptrs.Right = NewRight;
			Ptrs.Parent = NewParent;
			Ptrs.Color = NewColor;
			Ptrs.Data = NewData;

			Capacity = NewCapacity;
			Memory = NewMemory;
			MemorySizeInBytes = TotalSize;
		}

		template <typename T>
		static size_t SubAllocate(size_t& Offset, size_t Count)
		{
			size_t Base = AlignUp(Offset, alignof(T));
			Offset = Base + sizeof(T) * Count;
			return Base;
		}

		static constexpr size_t NumColorWords(size_t Count)
		{
			return (Count + NumColorWordBits - 1) / NumColorWordBits;
		}
	};

	NodeIndex Root = InvalidNode;
	FNodes Nodes;

	// Returns the index of the next sequential node after Node.
	NodeIndex Next(NodeIndex Node) const
	{
		// If Node has a right child, then the next node is the left-most
		// descendant of that node (or that node, if it has no descendants).
		if (NodeIndex Right = Nodes.Right(Node); Right != InvalidNode)
		{
			return LeftMostDescendant(Right);
		}

		// Otherwise the next node is the first ancestor where Node is under its left branch.
		for (NodeIndex Parent = Nodes.Parent(Node); Parent != InvalidNode; Parent = Nodes.Parent(Node))
		{
			if (Nodes.Left(Parent) == Node)
			{
				return Parent;
			}
			Node = Parent;
		}

		return InvalidNode;
	}

	// Returns the left-most leaf descendant of Node.
	NodeIndex LeftMostDescendant(NodeIndex Node) const
	{
		for (NodeIndex Left = Nodes.Left(Node); Left != InvalidNode; Left = Nodes.Left(Node))
		{
			Node = Left;
		}
		return Node;
	}

	// Returns the rightmost node whose Start < SearchStart, or InvalidNode.
	NodeIndex Predecessor(uintptr_t SearchStart) const
	{
		NodeIndex Result = InvalidNode;
		NodeIndex Current = Root;
		while (Current != InvalidNode)
		{
			if (Nodes.Interval(Current).Start < SearchStart)
			{
				Result = Current;
				Current = Nodes.Right(Current);
			}
			else
			{
				Current = Nodes.Left(Current);
			}
		}
		return Result;
	}

	// Returns the leftmost node whose Start >= SearchStart, or InvalidNode.
	NodeIndex LowerBound(uintptr_t SearchStart) const
	{
		NodeIndex Result = InvalidNode;
		NodeIndex Current = Root;
		while (Current != InvalidNode)
		{
			if (Nodes.Interval(Current).Start >= SearchStart)
			{
				Result = Current;
				Current = Nodes.Left(Current);
			}
			else
			{
				Current = Nodes.Right(Current);
			}
		}
		return Result;
	}

	// Left-rotates around X:  X's right child Y takes X's place.
	void RotateLeft(NodeIndex X)
	{
		const NodeIndex Y = Nodes.Right(X);
		AUTORTFM_ASSERT(Y != InvalidNode);

		Nodes.Right(X) = Nodes.Left(Y);
		if (Nodes.Left(Y) != InvalidNode)
		{
			Nodes.Parent(Nodes.Left(Y)) = X;
		}

		Nodes.Parent(Y) = Nodes.Parent(X);
		if (Nodes.Parent(X) == InvalidNode)
		{
			Root = Y;
		}
		else if (X == Nodes.Left(Nodes.Parent(X)))
		{
			Nodes.Left(Nodes.Parent(X)) = Y;
		}
		else
		{
			Nodes.Right(Nodes.Parent(X)) = Y;
		}

		Nodes.Left(Y) = X;
		Nodes.Parent(X) = Y;
	}

	// Right-rotates around X:  X's left child Y takes X's place.
	void RotateRight(NodeIndex X)
	{
		const NodeIndex Y = Nodes.Left(X);
		AUTORTFM_ASSERT(Y != InvalidNode);

		Nodes.Left(X) = Nodes.Right(Y);
		if (Nodes.Right(Y) != InvalidNode)
		{
			Nodes.Parent(Nodes.Right(Y)) = X;
		}

		Nodes.Parent(Y) = Nodes.Parent(X);
		if (Nodes.Parent(X) == InvalidNode)
		{
			Root = Y;
		}
		else if (X == Nodes.Right(Nodes.Parent(X)))
		{
			Nodes.Right(Nodes.Parent(X)) = Y;
		}
		else
		{
			Nodes.Left(Nodes.Parent(X)) = Y;
		}

		Nodes.Right(Y) = X;
		Nodes.Parent(X) = Y;
	}

	// Removes node Z from the red-black tree.
	// The node's slot in the SoA array is left as a dead entry (not reclaimed).
	UE_AUTORTFM_FORCENOINLINE void Remove(NodeIndex Z)
	{
		// If Z has two children, swap it with its in-order successor S.
		// S is the leftmost node of Z's right subtree, so it has at most one
		// child (a right child).  We copy S's data into Z and then delete S
		// from the tree structure instead.
		if (Nodes.Left(Z) != InvalidNode && Nodes.Right(Z) != InvalidNode)
		{
			const NodeIndex S = LeftMostDescendant(Nodes.Right(Z));
			Nodes.Interval(Z) = Nodes.Interval(S);
			Nodes.Data(Z) = std::move(Nodes.Data(S));
			Z = S;
		}

		// Z now has at most one non-null child.
		const NodeIndex C = (Nodes.Left(Z) != InvalidNode) ? Nodes.Left(Z) : Nodes.Right(Z);
		const NodeIndex P = Nodes.Parent(Z);

		// Splice Z out of the tree.
		if (C != InvalidNode)
		{
			Nodes.Parent(C) = P;
		}
		if (P == InvalidNode)
		{
			Root = C;
		}
		else if (Z == Nodes.Left(P))
		{
			Nodes.Left(P) = C;
		}
		else
		{
			Nodes.Right(P) = C;
		}

		// Color fixup.
		if (Nodes.Color(Z) == EColor::Red)
		{
			// Removing a red node never changes black height.
			return;
		}
		if (C != InvalidNode && Nodes.Color(C) == EColor::Red)
		{
			// The replacement is red: recolor it black to restore black height.
			Nodes.SetColor(C, EColor::Black);
			return;
		}

		// C is "double-black".  Walk up the tree applying the standard
		// six-case RB fixup until the extra black is absorbed.
		NodeIndex X = C;        // double-black node (may be InvalidNode = null)
		NodeIndex XParent = P;  // parent of X

		while (XParent != InvalidNode && (X == InvalidNode || Nodes.Color(X) == EColor::Black))
		{
			const bool bXIsLeft = (Nodes.Left(XParent) == X);
			NodeIndex W = bXIsLeft ? Nodes.Right(XParent) : Nodes.Left(XParent);  // sibling

			// Case 1: sibling W is red — rotate to make W black.
			if (W != InvalidNode && Nodes.Color(W) == EColor::Red)
			{
				Nodes.SetColor(W, EColor::Black);
				Nodes.SetColor(XParent, EColor::Red);
				if (bXIsLeft)
				{
					RotateLeft(XParent);
				}
				else
				{
					RotateRight(XParent);
				}
				W = bXIsLeft ? Nodes.Right(XParent) : Nodes.Left(XParent);
			}

			// W is now black (or null).
			const NodeIndex WFar = (W != InvalidNode) ? (bXIsLeft ? Nodes.Right(W) : Nodes.Left(W)) : InvalidNode;
			const NodeIndex WNear = (W != InvalidNode) ? (bXIsLeft ? Nodes.Left(W) : Nodes.Right(W)) : InvalidNode;
			const bool bWFarBlack = (WFar == InvalidNode || Nodes.Color(WFar) == EColor::Black);
			const bool bWNearBlack = (WNear == InvalidNode || Nodes.Color(WNear) == EColor::Black);

			if (bWFarBlack && bWNearBlack)
			{
				// Case 2: both of W's children are black — push the double-black up.
				if (W != InvalidNode)
				{
					Nodes.SetColor(W, EColor::Red);
				}
				if (Nodes.Color(XParent) == EColor::Red)
				{
					Nodes.SetColor(XParent, EColor::Black);
					return;  // resolved
				}
				X = XParent;
				XParent = Nodes.Parent(XParent);
				continue;
			}

			if (bWFarBlack)
			{
				// Case 3: near child is red, far child is black — rotate W to
				// make the far child red, then fall through to case 4.
				if (WNear != InvalidNode)
				{
					Nodes.SetColor(WNear, EColor::Black);
				}
				if (W != InvalidNode)
				{
					Nodes.SetColor(W, EColor::Red);
				}
				if (bXIsLeft)
				{
					RotateRight(W);
				}
				else
				{
					RotateLeft(W);
				}
				W = bXIsLeft ? Nodes.Right(XParent) : Nodes.Left(XParent);
			}

			// Case 4: far child is red — final rotation resolves the double-black.
			const NodeIndex NewWFar = (W != InvalidNode) ? (bXIsLeft ? Nodes.Right(W) : Nodes.Left(W)) : InvalidNode;
			Nodes.SetColor(W, Nodes.Color(XParent));
			Nodes.SetColor(XParent, EColor::Black);
			if (NewWFar != InvalidNode)
			{
				Nodes.SetColor(NewWFar, EColor::Black);
			}
			if (bXIsLeft)
			{
				RotateLeft(XParent);
			}
			else
			{
				RotateRight(XParent);
			}
			return;  // case 4 always terminates
		}

		// X has reached the root or is red: color it black to absorb the extra black.
		if (X != InvalidNode)
		{
			Nodes.SetColor(X, EColor::Black);
		}
	}

	// Inserts NewInterval with NewData into the tree, applying the ownership semantics:
	//   Phase A — extend MergedStart left by consuming adjacent/overlapping same-data predecessors;
	//             trim or split a different-data predecessor that overlaps.
	//   Phase B — scan forward from MergedStart: consume same-data nodes (extending MergedEnd),
	//             delete fully-covered different-data nodes, trim a right-partial overlap.
	//   Phase C — SimpleInsert the effective [MergedStart, MergedEnd) interval.
	//   Phase D — re-insert any deferred right-remainder created by a split or right trim.
	UE_AUTORTFM_FORCENOINLINE void Insert(FInterval NewInterval, DataType NewData)
	{
		uintptr_t MergedStart = NewInterval.Start;
		uintptr_t MergedEnd = NewInterval.End;

		// Storage for an optional deferred re-insertion (at most one can arise).
		// Placed in aligned raw storage to avoid requiring DataType to be default-constructible.
		bool bHasRemainder = false;
		FInterval RemainderInterval{uintptr_t{0}, uintptr_t{0}};
		alignas(DataType) std::byte RemainderDataBuf[sizeof(DataType)];
		auto SetRemainder = [&](FInterval Interval, const DataType& Data)
		{
			AUTORTFM_ASSERT(!bHasRemainder);
			bHasRemainder = true;
			RemainderInterval = Interval;
			::new (RemainderDataBuf) DataType(Data);
		};

		// === Phase A: handle the predecessor of MergedStart ===
		for (;;)
		{
			const NodeIndex Left = Predecessor(MergedStart);
			if (Left == InvalidNode)
			{
				break;
			}

			const FInterval LeftInterval = Nodes.Interval(Left);  // value copy — safe across modifications
			if (LeftInterval.End < MergedStart)
			{
				break;  // no contact
			}

			if (LeftInterval.End == MergedStart)
			{
				// Left-adjacent.
				if (Nodes.Data(Left) == NewData)
				{
					MergedStart = LeftInterval.Start;
					Remove(Left);
					continue;  // re-check new predecessor
				}
				break;  // different data adjacent: leave Left as-is
			}

			// LeftInterval.End > MergedStart: Left overlaps from the left.
			if (Nodes.Data(Left) == NewData)
			{
				MergedStart = LeftInterval.Start;
				if (LeftInterval.End > MergedEnd)
				{
					MergedEnd = LeftInterval.End;
				}  // extend right too
				Remove(Left);
				continue;
			}

			// Different data: trim Left's End in-place (End-only change; BST-safe).
			if (LeftInterval.End > MergedEnd)
			{
				// Left straddles our whole interval — save its right portion for Phase D.
				SetRemainder(FInterval{MergedEnd, LeftInterval.End}, Nodes.Data(Left));
			}
			Nodes.Interval(Left).End = MergedStart;
			break;
		}

		// === Phase B: handle nodes starting at or after MergedStart ===
		// Re-queries LowerBound each iteration because Remove() may rearrange the
		// BST (swapping a node with its in-order successor), which can invalidate
		// any previously-computed Next() pointer.
		for (;;)
		{
			const NodeIndex Right = LowerBound(MergedStart);
			if (Right == InvalidNode)
			{
				break;
			}

			const FInterval RightInterval = Nodes.Interval(Right);  // value copy
			if (RightInterval.Start > MergedEnd)
			{
				break;  // past our interval
			}

			if (RightInterval.Start == MergedEnd)
			{
				// Right-adjacent.
				if (Nodes.Data(Right) == NewData)
				{
					MergedEnd = RightInterval.End;
					Remove(Right);
				}
				break;  // stop regardless (diff data adjacent: leave Right as-is)
			}

			// RightInterval.Start < MergedEnd: Right overlaps our effective interval.
			if (Nodes.Data(Right) == NewData)
			{
				if (RightInterval.End > MergedEnd)
				{
					MergedEnd = RightInterval.End;
				}
				Remove(Right);
				continue;
			}

			// Different data: clip Right.
			if (RightInterval.End <= MergedEnd)
			{
				// Fully contained: new interval takes ownership — delete Right.
				Remove(Right);
				continue;
			}

			// Right extends past MergedEnd: keep [MergedEnd, RightInterval.End) with RData.
			// We cannot change Right.Start in-place (BST order violation), so defer.
			SetRemainder(FInterval{MergedEnd, RightInterval.End}, Nodes.Data(Right));
			Remove(Right);
			break;
		}

		// === Phase C: insert the (possibly extended) effective interval ===
		SimpleInsert(FInterval{MergedStart, MergedEnd}, std::move(NewData));

		// === Phase D: re-insert any deferred right remainder ===
		if (bHasRemainder)
		{
			auto* RemainderData = reinterpret_cast<DataType*>(RemainderDataBuf);
			SimpleInsert(RemainderInterval, std::move(*RemainderData));
			RemainderData->~DataType();
		}
	}

	UE_AUTORTFM_FORCEINLINE NodeIndex AddNode(FInterval NewInterval, DataType&& Data, NodeIndex Parent = InvalidNode)
	{
		return Nodes.Push({
			.Interval = NewInterval,
			.Parent = Parent,
			.Color = (Parent == InvalidNode) ? EColor::Black : EColor::Red,
			.Data = std::move(Data),
		});
	}

	// Fast-path insert: assumes [NewInterval] has no overlapping or adjacent intervals
	// already in the tree.  Called by Insert() after it has cleared the way.
	UE_AUTORTFM_FORCENOINLINE void SimpleInsert(FInterval NewInterval, DataType Data)
	{
		if (AUTORTFM_UNLIKELY(InvalidNode == Root))
		{
			Root = AddNode(NewInterval, std::move(Data));
			return;
		}

		NodeIndex Current = Root;

		for (;;)
		{
			const FInterval Interval = Nodes.Interval(Current);

			// SimpleInsert requires no pre-existing overlaps.
			AUTORTFM_ASSERT(!((NewInterval.Start < Interval.End) && (Interval.Start < NewInterval.End)));

			if (NewInterval.Start < Interval.Start)
			{
				if (InvalidNode == Nodes.Left(Current))
				{
					const NodeIndex Index = AddNode(NewInterval, std::move(Data), Current);
					Nodes.Left(Current) = Index;
					Current = Index;
					break;
				}

				Current = Nodes.Left(Current);
			}
			else
			{
				if (InvalidNode == Nodes.Right(Current))
				{
					const NodeIndex Index = AddNode(NewInterval, std::move(Data), Current);
					Nodes.Right(Current) = Index;
					Current = Index;
					break;
				}

				Current = Nodes.Right(Current);
			}

			AUTORTFM_ASSERT(Root != Current);
		}

		auto IsBlack = [this](NodeIndex Index)
		{
			return (InvalidNode == Index) || (EColor::Black == Nodes.Color(Index));
		};

		for (;;)
		{
			NodeIndex Parent = Nodes.Parent(Current);

			UE_AUTORTFM_ASSUME(Current != Parent);

			// The root will always have a black parent, so this check covers both.
			if (Parent == Root)
			{
				Nodes.SetColor(Parent, EColor::Black);
				break;
			}
			else if (IsBlack(Parent))
			{
				break;
			}

			const NodeIndex GrandParent = Nodes.Parent(Parent);

			UE_AUTORTFM_ASSUME((Parent != GrandParent) && (Current != GrandParent));

			const bool bParentIsLeft = Nodes.Left(GrandParent) == Parent;

			// The uncle is the other node of our parent.
			const NodeIndex Uncle = bParentIsLeft ? Nodes.Right(GrandParent) : Nodes.Left(GrandParent);

			UE_AUTORTFM_ASSUME((GrandParent != Uncle) && (Parent != Uncle) && (Current != Uncle));

			if (!IsBlack(Uncle))
			{
				AUTORTFM_ASSERT(IsBlack(GrandParent));
				Nodes.SetColor(Parent, EColor::Black);
				Nodes.SetColor(Uncle, EColor::Black);

				if (GrandParent == Root)
				{
					break;
				}
				else
				{
					Nodes.SetColor(GrandParent, EColor::Red);
				}

				Current = GrandParent;
				continue;
			}

			// Our uncle is black, so we need to swizzle around.
			const bool bCurrentIsLeft = Nodes.Left(Parent) == Current;

			if (bParentIsLeft)
			{
				if (!bCurrentIsLeft)
				{
					Nodes.Right(Parent) = Nodes.Left(Current);

					if (InvalidNode != Nodes.Right(Parent))
					{
						Nodes.Parent(Nodes.Right(Parent)) = Parent;
					}

					Nodes.Parent(Parent) = Current;
					Nodes.Parent(Current) = GrandParent;
					Nodes.Left(Current) = Parent;
					Nodes.Left(GrandParent) = Current;
					std::swap(Parent, Current);
				}

				Nodes.Parent(Parent) = Nodes.Parent(GrandParent);
				Nodes.Left(GrandParent) = Nodes.Right(Parent);

				if (InvalidNode != Nodes.Left(GrandParent))
				{
					Nodes.Parent(Nodes.Left(GrandParent)) = GrandParent;
				}

				Nodes.Right(Parent) = GrandParent;
				Nodes.Parent(GrandParent) = Parent;

				const EColor bGrandParentColor = Nodes.Color(GrandParent);
				Nodes.SetColor(GrandParent, Nodes.Color(Parent));
				Nodes.SetColor(Parent, bGrandParentColor);
			}
			else
			{
				if (bCurrentIsLeft)
				{
					Nodes.Left(Parent) = Nodes.Right(Current);

					if (InvalidNode != Nodes.Left(Parent))
					{
						Nodes.Parent(Nodes.Left(Parent)) = Parent;
					}

					Nodes.Parent(Parent) = Current;
					Nodes.Parent(Current) = GrandParent;
					Nodes.Right(Current) = Parent;
					Nodes.Right(GrandParent) = Current;
					std::swap(Parent, Current);
				}

				Nodes.Parent(Parent) = Nodes.Parent(GrandParent);
				Nodes.Right(GrandParent) = Nodes.Left(Parent);

				if (InvalidNode != Nodes.Right(GrandParent))
				{
					Nodes.Parent(Nodes.Right(GrandParent)) = GrandParent;
				}

				Nodes.Left(Parent) = GrandParent;
				Nodes.Parent(GrandParent) = Parent;

				const EColor bGrandParentColor = Nodes.Color(GrandParent);
				Nodes.SetColor(GrandParent, Nodes.Color(Parent));
				Nodes.SetColor(Parent, bGrandParentColor);
			}

			if (InvalidNode == Nodes.Parent(Parent))
			{
				Root = Parent;
			}
			else
			{
				const NodeIndex GreatGrandParent = Nodes.Parent(Parent);

				if (GrandParent == Nodes.Left(GreatGrandParent))
				{
					Nodes.Left(GreatGrandParent) = Parent;
				}
				else
				{
					Nodes.Right(GreatGrandParent) = Parent;
				}
			}

			break;
		}
	}
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
