// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM/Defines.h"
#include "BlockAllocator.h"

#include <cstddef>
#include <utility>

namespace AutoRTFM
{

template <typename ItemType, size_t InlineCapacity>
class AUTORTFM_INTERNAL TIntrusivePool final
{
public:
	using FItem = ItemType;

	TIntrusivePool() = default;

	~TIntrusivePool()
	{
		while (FItem* const Item = FreeList)
		{
			FreeList = *Item->GetIntrusiveAddress();

			Item->~FItem();
		}
	}

	// Acquires a new Item from the pool. If the item was previously returned
	// to the pool, we'll call `Resurrect` on it with `Arguments`. Otherwise
	// create a new item with `Arguments`.
	template <typename... ArgumentTypes>
	FItem* Take(ArgumentTypes&&... Arguments)
	{
		FItem* Item = nullptr;

		if (FreeList)
		{
			// Unlink a free entry from the free list.
			Item = FreeList;
			FreeList = *(FreeList->GetIntrusiveAddress());

			Item->Resurrect(std::forward<ArgumentTypes>(Arguments)...);
		}
		else
		{
			// Free list is empty. Allocate another item.
			void* const Memory = Allocator.Allocate(EntrySize, EntryAlignment);
			NumAllocated++;

			Item = new (Memory) FItem(std::forward<ArgumentTypes>(Arguments)...);
		}

		AUTORTFM_ASSERT(NumInUse < NumAllocated);
		NumInUse++;

		return Item;
	}

	// Calls `Suppress` on `Item` and returns it to the pool to be reused.
	void Return(FItem* const Item)
	{
		// Suppress the item.
		Item->Suppress();

		// Place the entry onto the free list.
		*Item->GetIntrusiveAddress() = FreeList;
		FreeList = Item;

		AUTORTFM_ASSERT(NumInUse > 0);
		NumInUse--;
	}

private:
	static constexpr size_t EntrySize = sizeof(FItem);
	static constexpr size_t EntryAlignment = alignof(FItem);

	// The underlying allocator for the pool.
	TBlockAllocator<InlineCapacity * EntrySize, EntryAlignment> Allocator;
	// Free list.
	FItem* FreeList = nullptr;
	// Number of entries allocated.
	size_t NumAllocated = 0;
	// Number of entries currently in use.
	size_t NumInUse = 0;

	TIntrusivePool(TIntrusivePool&&) = delete;
	TIntrusivePool(const TIntrusivePool&) = delete;
	TIntrusivePool& operator=(const TIntrusivePool&) = delete;
	TIntrusivePool& operator=(TIntrusivePool&&) = delete;
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
