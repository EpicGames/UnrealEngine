// Copyright Epic Games, Inc. All Rights Reserved.

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "LongJump.h"

#if AUTORTFM_USE_SAVE_RESTORE_CONTEXT
namespace AutoRTFM
{

// https://learn.microsoft.com/en-us/cpp/build/x64-calling-convention?view=msvc-170#callercallee-saved-registers
// Registers that must be saved:
// RBX, RBP, RDI, RSI, RSP, R12, R13, R14, R15, and XMM6-XMM15
UE_AUTORTFM_NOAUTORTFM __attribute__((noinline)) __attribute((naked)) bool FLongJump::SaveContext(void* Buffer)
{
	__asm
	{
		// rcx: Buffer, rax: Return value

		// Save non-volatile registers
		mov     [rcx +   0], rsp
		mov     [rcx +   8], rbx
		mov     [rcx +  16], rbp
		mov     [rcx +  24], rdi
		mov     [rcx +  32], rsi
		mov     [rcx +  40], r12
		mov     [rcx +  48], r13
		mov     [rcx +  56], r14
		mov     [rcx +  64], r15
		movupd  [rcx +  80], xmm6
		movupd  [rcx +  96], xmm7
		movupd  [rcx + 112], xmm8
		movupd  [rcx + 128], xmm9
		movupd  [rcx + 144], xmm10
		movupd  [rcx + 160], xmm11
		movupd  [rcx + 176], xmm12
		movupd  [rcx + 192], xmm13
		movupd  [rcx + 208], xmm14
		movupd  [rcx + 224], xmm15

			// Save register control & status state
		stmxcsr [rcx + 240]
		fnstcw  [rcx + 244]

		// Use return address on stack as restore instruction pointer
		mov     r8,          [rsp]
		mov     [rcx + 248], r8

			// Return 0 to indicate that this is a context-save
		mov     rax, 0
		ret
	}
}
UE_AUTORTFM_NOAUTORTFM __attribute__((noinline)) __attribute((naked)) void FLongJump::RestoreContext(void* Buffer)
{
	__asm
	{
		// rcx: Buffer, rax: Return value

		// Restore non-volatile registers
		mov     rsp,   [rcx +   0]
		mov     rbx,   [rcx +   8]
		mov     rbp,   [rcx +  16]
		mov     rdi,   [rcx +  24]
		mov     rsi,   [rcx +  32]
		mov     r12,   [rcx +  40]
		mov     r13,   [rcx +  48]
		mov     r14,   [rcx +  56]
		mov     r15,   [rcx +  64]
		movupd  xmm6,  [rcx +  80]
		movupd  xmm7,  [rcx +  96]
		movupd  xmm8,  [rcx + 112]
		movupd  xmm9,  [rcx + 128]
		movupd  xmm10, [rcx + 144]
		movupd  xmm11, [rcx + 160]
		movupd  xmm12, [rcx + 176]
		movupd  xmm13, [rcx + 192]
		movupd  xmm14, [rcx + 208]
		movupd  xmm15, [rcx + 224]

		// Restore register control & status state
		ldmxcsr        [rcx + 240]
		fldcw          [rcx + 244]
		fnclex

			// Replace caller's return address with saved return address and return.
		mov     r8,    [rcx + 248]
		mov     [rsp], r8

			// Return 1 to indicate that this is a context-restore
		mov     rax, 1
		ret
	}
}

}

#endif  // AUTORTFM_USE_SAVE_RESTORE_CONTEXT

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
