// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "BuildMacros.h"
#include "Utils.h"

#include <setjmp.h>

#if AUTORTFM_PLATFORM_WINDOWS && AUTORTFM_ARCHITECTURE_X64  // We only have an Windows X64 implementation right now.
#define AUTORTFM_USE_SAVE_RESTORE_CONTEXT 1
#else
#define AUTORTFM_USE_SAVE_RESTORE_CONTEXT 0
#endif

namespace AutoRTFM
{

// setjmp/longjmp that doesn't do any unwinding (no C++ destructor calls, no messing with OS
// signal states - just saving/restoring CPU state). Although it's the setjmp/longjmp everyone
// knows and loves, it's exposed as a try/catch/throw API to make it less error-prone.
class AUTORTFM_DISABLE FLongJump
{
public:
	FLongJump()
	{
		memset(this, 0, sizeof(*this));
	}

	template <typename TTryFunctor, typename TCatchFunctor>
	void TryCatch(const TTryFunctor& TryFunctor, const TCatchFunctor& CatchFunctor);

	[[noreturn]] void Throw();

private:
#if AUTORTFM_USE_SAVE_RESTORE_CONTEXT
	static bool SaveContext(void* Buffer);
	[[noreturn]] static void RestoreContext(void* Context);
	static constexpr size_t ContextSize = 256;
	static constexpr size_t ContextAlignment = 16;
	alignas(ContextAlignment) std::byte Context[ContextSize];
#else
	jmp_buf JmpBuf;
#endif
	bool bIsSet;
};

template <typename TTryFunctor, typename TCatchFunctor>
void FLongJump::TryCatch(const TTryFunctor& TryFunctor, const TCatchFunctor& CatchFunctor)
{
	AUTORTFM_ASSERT(!bIsSet);
#if AUTORTFM_USE_SAVE_RESTORE_CONTEXT
	if (!SaveContext(Context))
#elif AUTORTFM_PLATFORM_WINDOWS
	if (!setjmp(JmpBuf))
#else
	if (!_setjmp(JmpBuf))
#endif
	{
		bIsSet = true;
		TryFunctor();
		bIsSet = false;
	}
	else
	{
		AUTORTFM_ASSERT(bIsSet);
		bIsSet = false;
		CatchFunctor();
	}
}

inline void FLongJump::Throw()
{
	AUTORTFM_ASSERT(bIsSet);
#if AUTORTFM_USE_SAVE_RESTORE_CONTEXT
	RestoreContext(Context);
#elif AUTORTFM_PLATFORM_WINDOWS
	longjmp(JmpBuf, 1);
#else
	_longjmp(JmpBuf, 1);
#endif
}

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
