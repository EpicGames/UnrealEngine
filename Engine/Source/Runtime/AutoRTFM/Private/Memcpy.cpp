// Copyright Epic Games, Inc. All Rights Reserved.

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Memcpy.h"

#include "ContextInlines.h"
#include "Utils.h"

namespace AutoRTFM
{

void* MemcpyToNew(void* InDst, const void* InSrc, size_t Size, FContext* Context)
{
	AUTORTFM_VERBOSE("MemcpyToNew(%p, %p, %zu)", InDst, InSrc, Size);

	// This will eventually go away when `n3322` is adopted, but for now we workaround
	// this by checking for `Size == 0` and bailing early (the only defined behavior
	// way going forward).
	if (AUTORTFM_UNLIKELY(Size == 0))
	{
		return nullptr;
	}

	return memcpy(InDst, InSrc, Size);
}

void* Memcpy(void* InDst, const void* InSrc, size_t Size, FContext* Context)
{
	AUTORTFM_VERBOSE("Memcpy(%p, %p, %zu)", InDst, InSrc, Size);

	// This will eventually go away when `n3322` is adopted, but for now we workaround
	// this by checking for `Size == 0` and bailing early (the only defined behavior
	// way going forward).
	if (AUTORTFM_UNLIKELY(Size == 0))
	{
		return nullptr;
	}

	Context->RecordWrite(InDst, Size);
	return memcpy(InDst, InSrc, Size);
}

void* Memmove(void* InDst, const void* InSrc, size_t Size, FContext* Context)
{
	AUTORTFM_VERBOSE("Memmove(%p, %p, %zu)", InDst, InSrc, Size);

	// This will eventually go away when `n3322` is adopted, but for now we workaround
	// this by checking for `Size == 0` and bailing early (the only defined behavior
	// way going forward).
	if (AUTORTFM_UNLIKELY(Size == 0))
	{
		return nullptr;
	}

	Context->RecordWrite(InDst, Size);
	return memmove(InDst, InSrc, Size);
}

void* Memset(void* InDst, int Value, size_t Size, FContext* Context)
{
	AUTORTFM_VERBOSE("Memset(%p, %d, %zu)", InDst, Value, Size);

	// This will eventually go away when `n3322` is adopted, but for now we workaround
	// this by checking for `Size == 0` and bailing early (the only defined behavior
	// way going forward).
	if (AUTORTFM_UNLIKELY(Size == 0))
	{
		return nullptr;
	}

	Context->RecordWrite(InDst, Size);
	return memset(InDst, Value, Size);
}

}  // namespace AutoRTFM

#endif  // defined(__AUTORTFM) && __AUTORTFM
