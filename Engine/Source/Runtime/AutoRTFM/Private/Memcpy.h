// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Utils.h"

#include <stdlib.h>

namespace AutoRTFM
{

class FContext;

AUTORTFM_INTERNAL void* MemcpyToNew(void* Dst, const void* Src, size_t Size, FContext* Context);
AUTORTFM_INTERNAL void* Memcpy(void* Dst, const void* Src, size_t Size, FContext* Context);
AUTORTFM_INTERNAL void* Memmove(void* Dst, const void* Src, size_t Size, FContext* Context);
AUTORTFM_INTERNAL void* Memset(void* Dst, int Value, size_t Size, FContext* Context);

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
