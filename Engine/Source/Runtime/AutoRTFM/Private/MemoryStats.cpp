// Copyright Epic Games, Inc. All Rights Reserved.

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "MemoryStats.h"
#include "Utils.h"

namespace AutoRTFM
{

std::atomic<size_t> FMemoryStats::BytesAllocated = 0;

#if AUTORTFM_CHECK_FOR_ALLOCATION_UNDERFLOW
void FMemoryStats::FatalErrorBytesAllocatedUnderflow()
{
	AUTORTFM_FATAL("FMemoryStats::BytesAllocated has underflowed");
}
#endif

}  // namespace AutoRTFM

#endif  // defined(__AUTORTFM) && __AUTORTFM
