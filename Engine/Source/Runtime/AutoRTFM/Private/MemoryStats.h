// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM.h"
#include "BuildMacros.h"
#include <atomic>

namespace AutoRTFM
{

#if AUTORTFM_BUILD_SHIPPING
#define AUTORTFM_CHECK_FOR_ALLOCATION_UNDERFLOW 0
#else
#define AUTORTFM_CHECK_FOR_ALLOCATION_UNDERFLOW 1
#endif

struct FMemoryStats
{
	// size_t constant with only the most-significant-bit set.
	// Used to quickly check for underflow/overflow, as we don't expect over
	// nine quintillion bytes of heap memory to be allocated.
	static constexpr size_t UnderflowMask = ((~static_cast<size_t>(0)) >> 1) + 1;

	/** Adjusts the total number of bytes allocated by Delta. */
	static inline void UpdateBytesAllocated(ptrdiff_t Delta)
	{
		[[maybe_unused]] size_t const NewBytesAllocated = (BytesAllocated += Delta);

#if AUTORTFM_CHECK_FOR_ALLOCATION_UNDERFLOW
		if (AUTORTFM_UNLIKELY(NewBytesAllocated & FMemoryStats::UnderflowMask))
		{
			FatalErrorBytesAllocatedUnderflow();
		}
#endif
	}

	/** Returns the total number of heap bytes allocated by AutoRTFM. */
	static inline size_t GetTotalBytesAllocated()
	{
		return BytesAllocated;
	}

private:
#if AUTORTFM_CHECK_FOR_ALLOCATION_UNDERFLOW
	UE_AUTORTFM_API static void FatalErrorBytesAllocatedUnderflow();  // Not inline to break circular header dependencies.
#endif

	UE_AUTORTFM_API static std::atomic<size_t> BytesAllocated;
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
