// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Allocator.h"
#include "ContainerValidation.h"
#include "Interval.h"
#include "Stack.h"
#include "Utils.h"

#include <cstring>

namespace AutoRTFM
{

// FMemoryTracker tracks a set of non-overlapping memory intervals using a red-black tree.
// Each interval is stored as [Start, End) and intervals that touch at their endpoints are
// merged into a single interval automatically on insert.
//
// Assumes that queries never partially overlap a stored interval — a query either falls
// entirely within an interval or entirely outside it. This assumption enables a simplified
// BST traversal in Contains() that avoids examining both subtrees.
struct AUTORTFM_INTERNAL FMemoryTracker final
{
private:
	using NodeIndex = uint32_t;
	static constexpr NodeIndex InvalidNode = UINT32_MAX;

public:
	FMemoryTracker() = default;

	FMemoryTracker(const FMemoryTracker&) = delete;
	FMemoryTracker& operator=(const FMemoryTracker&) = delete;

	// Inserts the interval [Address, Address+Size) into the tracker.
	// If the new interval is adjacent to an existing interval, the two are merged.
	// Returns false if the interval overlaps (but is not adjacent to) an existing interval.
	bool Insert(const void* const Address, const size_t Size)
	{
		return Insert(FInterval{Address, Size});
	}

	// Returns true if the interval [Address, Address+Size) overlaps any stored interval.
	bool Contains(const void* const Address, const size_t Size) const
	{
		return Query(Address, Size);
	}

	// Returns true if no intervals are stored.
	bool IsEmpty() const
	{
		return InvalidNode == Root;
	}

	// Removes all stored intervals.
	void Reset()
	{
		Root = InvalidNode;
		Nodes.Reset();
	}

	// Inserts all intervals from Other into this tracker.
	void Merge(const FMemoryTracker& Other)
	{
		if (Other.IsEmpty())
		{
			return;
		}

		TStack<NodeIndex, 32, FAllocator, EContainerValidation::Disabled> ToProcess;

		ToProcess.Push(Other.Root);

		do
		{
			const NodeIndex Current = ToProcess.Back();
			ToProcess.Pop();

			Insert(Other.Nodes.Interval(Current));

			const NodeIndex Left = Other.Nodes.Left(Current);
			const NodeIndex Right = Other.Nodes.Right(Current);

			if (InvalidNode != Left)
			{
				ToProcess.Push(Left);
			}

			if (InvalidNode != Right)
			{
				ToProcess.Push(Right);
			}
		} while (!ToProcess.IsEmpty());
	}

	// A forward iterator of the FMemoryTracker
	struct FIterator
	{
		FIterator(const FIterator&) = default;
		FIterator& operator=(const FIterator&) = default;

		FInterval operator*() const
		{
			return Tree->Nodes.Interval(Node);
		}

		FIterator& operator++()
		{
			Node = Tree->Next(Node);
			return *this;
		}

		bool operator==(const FIterator& Other) const = default;
		bool operator!=(const FIterator& Other) const = default;

	private:
		friend struct FMemoryTracker;
		FIterator(FMemoryTracker const* Tree, NodeIndex Node) : Tree{Tree}, Node{Node} {}
		FMemoryTracker const* Tree = nullptr;
		NodeIndex Node = InvalidNode;
	};

	// Returns an iterator to the first interval in the tree
	FIterator begin() const
	{
		return FIterator(this, Root == InvalidNode ? InvalidNode : LeftMostDescendant(Root));
	}

	// Returns an end iterator for the tree
	FIterator end() const
	{
		return FIterator(this, InvalidNode);
	}

private:
	friend struct FIterator;

	enum EColor : bool
	{
		Red = false,
		Black = true
	};

	using ColorWord = uint64_t;

	struct FNode final
	{
		FInterval Interval;
		NodeIndex Left = InvalidNode;
		NodeIndex Right = InvalidNode;
		NodeIndex Parent = InvalidNode;
		EColor Color;
	};

	struct FNodePtrs final
	{
		FInterval* Interval = nullptr;
		NodeIndex* Left = nullptr;
		NodeIndex* Right = nullptr;
		NodeIndex* Parent = nullptr;
		ColorWord* Color = nullptr;
	};

	// A list of FNodes, stored as a structure-of-arrays.
	struct FNodes final
	{
		static constexpr size_t NumColorWordBits = sizeof(ColorWord) * 8;
		static constexpr NodeIndex InitialCapacity = 32;

		FNodePtrs Ptrs;
		NodeIndex Count = 0;
		NodeIndex Capacity = 0;

		FNodes() = default;

		~FNodes()
		{
			Reset();
		}

		// Clears the node list to an empty state.
		void Reset()
		{
			Ptrs = {};
			Count = 0;
			Capacity = 0;

			FAllocator::Free(Memory, MemorySizeInBytes);
			Memory = nullptr;
			MemorySizeInBytes = 0;
		}

		// Pushes a new node to the end of the list.
		// Returns the node index.
		NodeIndex Push(FNode Node)
		{
			if (Count >= Capacity)
			{
				Resize(std::max(Capacity * 2, InitialCapacity));
			}

			Ptrs.Interval[Count] = Node.Interval;
			Ptrs.Left[Count] = Node.Left;
			Ptrs.Right[Count] = Node.Right;
			Ptrs.Parent[Count] = Node.Parent;
			SetColor(Count, Node.Color);

			return static_cast<NodeIndex>(Count++);
		}

		FInterval& Interval(NodeIndex Index)
		{
			return Ptrs.Interval[Index];
		}
		const FInterval& Interval(NodeIndex Index) const
		{
			return Ptrs.Interval[Index];
		}
		NodeIndex& Left(NodeIndex Index)
		{
			return Ptrs.Left[Index];
		}
		const NodeIndex& Left(NodeIndex Index) const
		{
			return Ptrs.Left[Index];
		}
		NodeIndex& Right(NodeIndex Index)
		{
			return Ptrs.Right[Index];
		}
		const NodeIndex& Right(NodeIndex Index) const
		{
			return Ptrs.Right[Index];
		}
		NodeIndex& Parent(NodeIndex Index)
		{
			return Ptrs.Parent[Index];
		}
		const NodeIndex& Parent(NodeIndex Index) const
		{
			return Ptrs.Parent[Index];
		}

		void SetColor(NodeIndex Index, EColor Color)
		{
			ColorWord const Bit = static_cast<ColorWord>(1) << (Index & (NumColorWordBits - 1));
			ColorWord& Word = Ptrs.Color[Index / NumColorWordBits];
			if (Color == EColor::Black)
			{
				Word |= Bit;
			}
			else
			{
				Word &= ~Bit;
			}
		}

		EColor Color(size_t Index) const
		{
			ColorWord const Bit = static_cast<ColorWord>(1) << (Index & (NumColorWordBits - 1));
			ColorWord& Word = Ptrs.Color[Index / NumColorWordBits];
			return (Word & Bit) ? EColor::Black : EColor::Red;
		}

	private:
		std::byte* Memory = nullptr;
		size_t MemorySizeInBytes = 0;

		FNodes(const FNodes&) = delete;
		FNodes& operator=(const FNodes&) = delete;

		void Resize(NodeIndex NewCapacity)
		{
			size_t TotalSize = 0;
			size_t const IntervalOffset = SubAllocate<FInterval>(TotalSize, NewCapacity);
			size_t const LeftOffset = SubAllocate<NodeIndex>(TotalSize, NewCapacity);
			size_t const RightOffset = SubAllocate<NodeIndex>(TotalSize, NewCapacity);
			size_t const ParentOffset = SubAllocate<NodeIndex>(TotalSize, NewCapacity);
			size_t const ColorOffset = SubAllocate<ColorWord>(TotalSize, NumColorWords(NewCapacity));
			auto* const NewMemory = reinterpret_cast<std::byte*>(FAllocator::Allocate(TotalSize, alignof(FNode)));
			auto* const NewInterval = std::launder(reinterpret_cast<FInterval*>(NewMemory + IntervalOffset));
			auto* const NewLeft = std::launder(reinterpret_cast<NodeIndex*>(NewMemory + LeftOffset));
			auto* const NewRight = std::launder(reinterpret_cast<NodeIndex*>(NewMemory + RightOffset));
			auto* const NewParent = std::launder(reinterpret_cast<NodeIndex*>(NewMemory + ParentOffset));
			auto* const NewColor = std::launder(reinterpret_cast<ColorWord*>(NewMemory + ColorOffset));
			if (size_t const NumToCopy = std::min(Count, NewCapacity))
			{
				memcpy(NewInterval, Ptrs.Interval, sizeof(FInterval) * NumToCopy);
				memcpy(NewLeft, Ptrs.Left, sizeof(NodeIndex) * NumToCopy);
				memcpy(NewRight, Ptrs.Right, sizeof(NodeIndex) * NumToCopy);
				memcpy(NewParent, Ptrs.Parent, sizeof(NodeIndex) * NumToCopy);
				memcpy(NewColor, Ptrs.Color, sizeof(ColorWord) * NumColorWords(NumToCopy));
			}

			FAllocator::Free(Memory, MemorySizeInBytes);

			Ptrs.Interval = NewInterval;
			Ptrs.Left = NewLeft;
			Ptrs.Right = NewRight;
			Ptrs.Parent = NewParent;
			Ptrs.Color = NewColor;

			Capacity = NewCapacity;
			Memory = NewMemory;
			MemorySizeInBytes = TotalSize;
		}

		template <typename T>
		static size_t SubAllocate(size_t& Offset, size_t Count)
		{
			size_t Base = AlignUp(Offset, alignof(T));
			Offset = Base + sizeof(T) * Count;
			return Base;
		}

		static constexpr size_t NumColorWords(size_t Count)
		{
			return (Count + NumColorWordBits - 1) / NumColorWordBits;
		}
	};

	NodeIndex Root = InvalidNode;
	FNodes Nodes;

	static constexpr bool bExtraDebugging = false;

	// Returns the index of the next sequential node after Node.
	NodeIndex Next(NodeIndex Node) const
	{
		// If Node has a right child, then the next node is the left-most
		// descendant of that node (or that node, if it has no descendants).
		if (NodeIndex Right = Nodes.Right(Node); Right != InvalidNode)
		{
			return LeftMostDescendant(Right);
		}

		// Otherwise the next node is the first ancestor where Node is under its left branch.
		for (NodeIndex Parent = Nodes.Parent(Node); Parent != InvalidNode; Parent = Nodes.Parent(Node))
		{
			if (Nodes.Left(Parent) == Node)
			{
				return Parent;
			}
			Node = Parent;
		}

		return InvalidNode;
	}

	// Returns the left-most leaf descendant of Node.
	NodeIndex LeftMostDescendant(NodeIndex Node) const
	{
		for (NodeIndex Left = Nodes.Left(Node); Left != InvalidNode; Left = Nodes.Left(Node))
		{
			Node = Left;
		}
		return Node;
	}

	UE_AUTORTFM_FORCEINLINE NodeIndex AddNode(FInterval Interval, NodeIndex Parent = InvalidNode)
	{
		return Nodes.Push({
			.Interval = Interval,
			.Parent = Parent,
			.Color = (Parent == InvalidNode) ? EColor::Black : EColor::Red,
		});
	}

	UE_AUTORTFM_FORCENOINLINE bool Insert(FInterval NewInterval)
	{
		if (AUTORTFM_UNLIKELY(InvalidNode == Root))
		{
			AUTORTFM_ASSERT(Nodes.Count == 0);
			Root = AddNode(NewInterval);
			return true;
		}

		NodeIndex Current = Root;

		for (;;)
		{
			const FInterval Interval = Nodes.Interval(Current);

			if (AUTORTFM_UNLIKELY((NewInterval.Start < Interval.End) && (Interval.Start < NewInterval.End)))
			{
				return false;
			}

			if (NewInterval.Start < Interval.Start)
			{
				if (AUTORTFM_UNLIKELY(NewInterval.End == Interval.Start))
				{
					AUTORTFM_ASSERT(NewInterval.Start < Interval.Start);

					// We can just modify the existing node in place.
					Nodes.Interval(Current).Start = NewInterval.Start;
					return true;
				}
				else if (InvalidNode == Nodes.Left(Current))
				{
					const NodeIndex Index = AddNode(NewInterval, Current);
					Nodes.Left(Current) = Index;
					Current = Index;
					break;
				}

				Current = Nodes.Left(Current);
			}
			else
			{
				if (AUTORTFM_UNLIKELY(NewInterval.Start == Interval.End))
				{
					AUTORTFM_ASSERT(NewInterval.End > Interval.End);

					// We can just modify the existing node in place.
					Nodes.Interval(Current).End = NewInterval.End;
					return true;
				}
				else if (InvalidNode == Nodes.Right(Current))
				{
					const NodeIndex Index = AddNode(NewInterval, Current);
					Nodes.Right(Current) = Index;
					Current = Index;
					break;
				}

				Current = Nodes.Right(Current);
			}

			AUTORTFM_ASSERT(Root != Current);
		}

		auto IsBlack = [this](NodeIndex Index)
		{
			return (InvalidNode == Index) || (EColor::Black == Nodes.Color(Index));
		};

		for (;;)
		{
			NodeIndex Parent = Nodes.Parent(Current);

			UE_AUTORTFM_ASSUME(Current != Parent);

			// The root will always have a black parent, so this check covers both.
			if (Parent == Root)
			{
				Nodes.SetColor(Parent, EColor::Black);
				break;
			}
			else if (IsBlack(Parent))
			{
				break;
			}

			const NodeIndex GrandParent = Nodes.Parent(Parent);

			UE_AUTORTFM_ASSUME((Parent != GrandParent) && (Current != GrandParent));

			const bool bParentIsLeft = Nodes.Left(GrandParent) == Parent;

			// The uncle is the other node of our parent.
			const NodeIndex Uncle = bParentIsLeft ? Nodes.Right(GrandParent) : Nodes.Left(GrandParent);

			UE_AUTORTFM_ASSUME((GrandParent != Uncle) && (Parent != Uncle) && (Current != Uncle));

			if (!IsBlack(Uncle))
			{
				AUTORTFM_ASSERT(IsBlack(GrandParent));
				Nodes.SetColor(Parent, EColor::Black);
				Nodes.SetColor(Uncle, EColor::Black);

				if (GrandParent == Root)
				{
					break;
				}
				else
				{
					Nodes.SetColor(GrandParent, EColor::Red);
				}

				Current = GrandParent;
				continue;
			}

			// Our uncle is black, so we need to swizzle around.
			const bool bCurrentIsLeft = Nodes.Left(Parent) == Current;

			if (bParentIsLeft)
			{
				if (!bCurrentIsLeft)
				{
					Nodes.Right(Parent) = Nodes.Left(Current);

					if (InvalidNode != Nodes.Right(Parent))
					{
						Nodes.Parent(Nodes.Right(Parent)) = Parent;
					}

					Nodes.Parent(Parent) = Current;
					Nodes.Parent(Current) = GrandParent;
					Nodes.Left(Current) = Parent;
					Nodes.Left(GrandParent) = Current;
					std::swap(Parent, Current);
				}

				Nodes.Parent(Parent) = Nodes.Parent(GrandParent);
				Nodes.Left(GrandParent) = Nodes.Right(Parent);

				if (InvalidNode != Nodes.Left(GrandParent))
				{
					Nodes.Parent(Nodes.Left(GrandParent)) = GrandParent;
				}

				Nodes.Right(Parent) = GrandParent;
				Nodes.Parent(GrandParent) = Parent;

				const EColor bGrandParentColor = Nodes.Color(GrandParent);
				Nodes.SetColor(GrandParent, Nodes.Color(Parent));
				Nodes.SetColor(Parent, bGrandParentColor);
			}
			else
			{
				if (bCurrentIsLeft)
				{
					Nodes.Left(Parent) = Nodes.Right(Current);

					if (InvalidNode != Nodes.Left(Parent))
					{
						Nodes.Parent(Nodes.Left(Parent)) = Parent;
					}

					Nodes.Parent(Parent) = Current;
					Nodes.Parent(Current) = GrandParent;
					Nodes.Right(Current) = Parent;
					Nodes.Right(GrandParent) = Current;
					std::swap(Parent, Current);
				}

				Nodes.Parent(Parent) = Nodes.Parent(GrandParent);
				Nodes.Right(GrandParent) = Nodes.Left(Parent);

				if (InvalidNode != Nodes.Right(GrandParent))
				{
					Nodes.Parent(Nodes.Right(GrandParent)) = GrandParent;
				}

				Nodes.Left(Parent) = GrandParent;
				Nodes.Parent(GrandParent) = Parent;

				const EColor bGrandParentColor = Nodes.Color(GrandParent);
				Nodes.SetColor(GrandParent, Nodes.Color(Parent));
				Nodes.SetColor(Parent, bGrandParentColor);
			}

			if (InvalidNode == Nodes.Parent(Parent))
			{
				Root = Parent;
			}
			else
			{
				const NodeIndex GreatGrandParent = Nodes.Parent(Parent);

				if (GrandParent == Nodes.Left(GreatGrandParent))
				{
					Nodes.Left(GreatGrandParent) = Parent;
				}
				else
				{
					Nodes.Right(GreatGrandParent) = Parent;
				}
			}

			break;
		}

		AssertStructureIsOk();

		return true;
	}

	UE_AUTORTFM_FORCENOINLINE bool Query(const void* const Address, const size_t Size) const
	{
		if (AUTORTFM_UNLIKELY(InvalidNode == Root))
		{
			return false;
		}

		const FInterval QueryInterval{Address, Size};

		NodeIndex Current = Root;

		do
		{
			const FInterval Interval = Nodes.Interval(Current);

			// This check does not need to prove that QueryInterval is entirely
			// enclosed within Interval, because if any byte of QueryInterval was in the
			// original Interval then it **must** already have been new memory.
			if ((QueryInterval.Start < Interval.End) && (Interval.Start < QueryInterval.End))
			{
				return true;
			}

			const NodeIndex* Next = (QueryInterval.Start < Interval.End) ? Nodes.Ptrs.Left : Nodes.Ptrs.Right;
			Current = Next[Current];
		} while (InvalidNode != Current);

		return false;
	}

	UE_AUTORTFM_FORCEINLINE void AssertStructureIsOk() const
	{
		if constexpr (bExtraDebugging)
		{
			if (InvalidNode != Root)
			{
				AssertNodeIsOk(Root);
			}
		}
	}

	UE_AUTORTFM_FORCENOINLINE void AssertNodeIsOk(NodeIndex Index) const
	{
		// We need to use recursion to check because we cannot have any
		// allocations within the checker itself!
		if constexpr (bExtraDebugging)
		{
			AUTORTFM_ASSERT(InvalidNode != Index);
			AUTORTFM_ASSERT(Index < Nodes.Count);

			const NodeIndex Parent = Nodes.Parent(Index);
			const NodeIndex Left = Nodes.Left(Index);
			const NodeIndex Right = Nodes.Right(Index);

			if (InvalidNode == Parent)
			{
				AUTORTFM_ASSERT(Root == Index);
			}
			else
			{
				AUTORTFM_ASSERT(Nodes.Color(Parent) == EColor::Black || Nodes.Color(Index) == EColor::Black);
				AUTORTFM_ASSERT((Nodes.Left(Parent) == Index) ^ (Nodes.Right(Parent) == Index));
			}

			AUTORTFM_ASSERT(Left != Index);
			AUTORTFM_ASSERT(Right != Index);

			if (InvalidNode != Left)
			{
				AssertNodeIsOk(Left);
			}

			if (InvalidNode != Right)
			{
				AssertNodeIsOk(Right);
			}
		}
	}
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
