// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/Metrics.h"
#include "MetricsPriv.h"
#include "Utils.h"
#include <algorithm>  // for std::max
#include <cinttypes>  // for PRIu64
#include <string>

namespace AutoRTFM
{

#if (defined(__AUTORTFM) && __AUTORTFM)

FAutoRTFMMetrics GAutoRTFMMetrics;

void ResetAutoRTFMMetrics()
{
	GAutoRTFMMetrics = {};
}

FAutoRTFMMetrics GetAutoRTFMMetrics()
{
	return GAutoRTFMMetrics;
}

bool ConvertMetricsToCSV(const FAutoRTFMMetrics& Metrics, char* Buffer, int BufferLength)
{
	int Appended = snprintf(Buffer, BufferLength,
		"%" PRIu64 ",%" PRIu64 ",%" PRIu64 ",%" PRIu64 ",%" PRIu64 ",%" PRIu64 ",%" PRIu64 ",%" PRIu64 ",%" PRIu64 ",%" PRIu64,
		Metrics.NumTopLevelTransactionsCompleted, Metrics.NumTransactionsStarted, Metrics.NumTransactionsCommitted,
		Metrics.NumTransactionsAborted, Metrics.NumRequestedAborts, Metrics.NumRequestedCascadingAborts,
		Metrics.NumRequestedCascadingRetries, Metrics.NumTransactionalViolations, Metrics.OverallPeakMemoryUsage,
		Metrics.ActiveTransactionPeakMemoryUsage);
	if (UNLIKELY(Appended < 0 || Appended >= BufferLength))
	{
		return false;
	}

	Buffer += Appended;
	BufferLength -= Appended;

	for (uint32_t Entry : Metrics.MemoryUsageHistogram)
	{
		Appended = snprintf(Buffer, BufferLength, ",%" PRIu32, Entry);
		if (UNLIKELY(Appended < 0 || Appended >= BufferLength))
		{
			return false;
		}

		Buffer += Appended;
		BufferLength -= Appended;
	}

	return true;
}

void UpdatePeakMemoryUsageMetrics(size_t MemoryUsed)
{
	GAutoRTFMMetrics.OverallPeakMemoryUsage = std::max(GAutoRTFMMetrics.OverallPeakMemoryUsage, MemoryUsed);
	GAutoRTFMMetrics.ActiveTransactionPeakMemoryUsage = std::max(GAutoRTFMMetrics.ActiveTransactionPeakMemoryUsage, MemoryUsed);
}

void UpdateMemoryUsageHistogram()
{
	// Histogram buckets should saturate, not overflow.
	size_t BucketIndex = AutoRTFM::Log2(GAutoRTFMMetrics.ActiveTransactionPeakMemoryUsage);
	uint32_t& Bucket = GAutoRTFMMetrics.MemoryUsageHistogram[BucketIndex];
	Bucket += AUTORTFM_LIKELY(Bucket < UINT32_MAX);

	// This top-level transaction has completed, so increment the counter and zero out peak memory usage.
	GAutoRTFMMetrics.NumTopLevelTransactionsCompleted++;
	GAutoRTFMMetrics.ActiveTransactionPeakMemoryUsage = 0;
}

#else  // __AUTORTFM off

void ResetAutoRTFMMetrics() {}

FAutoRTFMMetrics GetAutoRTFMMetrics()
{
	return {};
}

#endif

}  // namespace AutoRTFM
