// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/CAPI.h"
#include "AutoRTFM/Metrics.h"
#include "MemoryStats.h"

#if (defined(__AUTORTFM) && __AUTORTFM)

namespace AutoRTFM
{

/** Converts an AutoRTFM metrics struct into CSV-formatted text. */
UE_AUTORTFM_API bool ConvertMetricsToCSV(const FAutoRTFMMetrics& Metrics, char* Buffer, int BufferLength);

/** Increments counter for number of transactions started. */
inline void IncrementNumTransactionsStartedMetric(size_t Amount = 1)
{
	extern AutoRTFM::FAutoRTFMMetrics GAutoRTFMMetrics;
	GAutoRTFMMetrics.NumTransactionsStarted += Amount;
}

/** Increments counter for number of transactions committed. */
inline void IncrementNumTransactionsCommittedMetric(size_t Amount = 1)
{
	extern AutoRTFM::FAutoRTFMMetrics GAutoRTFMMetrics;
	GAutoRTFMMetrics.NumTransactionsCommitted += Amount;
}

/** Increments counter for number of transactions aborted. */
inline void IncrementNumTransactionsAbortedMetric(size_t Amount = 1)
{
	extern AutoRTFM::FAutoRTFMMetrics GAutoRTFMMetrics;
	GAutoRTFMMetrics.NumTransactionsAborted += Amount;
}

/** Increments counter for number of calls to AbortTransaction. */
inline void IncrementNumRequestedAbortsMetric(size_t Amount = 1)
{
	extern AutoRTFM::FAutoRTFMMetrics GAutoRTFMMetrics;
	GAutoRTFMMetrics.NumRequestedAborts += Amount;
}

/** Increments counter for number of CascadingAbortTransaction calls. */
inline void IncrementNumRequestedCascadingAbortsMetric(size_t Amount = 1)
{
	extern AutoRTFM::FAutoRTFMMetrics GAutoRTFMMetrics;
	GAutoRTFMMetrics.NumRequestedCascadingAborts += Amount;
}

/** Increments counter for number of CascadingRetryTransaction calls. */
inline void IncrementNumRequestedCascadingRetriesMetric(size_t Amount = 1)
{
	extern AutoRTFM::FAutoRTFMMetrics GAutoRTFMMetrics;
	GAutoRTFMMetrics.NumRequestedCascadingRetries += Amount;
}

/** Increments counter for number of transactional aborts caused by violating transactionality (unsupported calls or API misuse). */
inline void IncrementNumTransactionalViolationsMetric(size_t Amount = 1)
{
	extern AutoRTFM::FAutoRTFMMetrics GAutoRTFMMetrics;
	GAutoRTFMMetrics.NumTransactionalViolations += Amount;
}

/** Updates the peak memory usage metrics with the passed-in value, if it's higher than the current peak value. */
void UpdatePeakMemoryUsageMetrics(size_t MemoryUsed = FMemoryStats::GetTotalBytesAllocated());

/** Updates the memory histogram metric, and resets the current-transaction peak memory usage metric. */
void UpdateMemoryUsageHistogram();

}  // namespace AutoRTFM

#endif