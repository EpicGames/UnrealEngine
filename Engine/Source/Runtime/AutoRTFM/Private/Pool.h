// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "BlockAllocator.h"

#include <cstddef>
#include <utility>

namespace AutoRTFM
{

// An allocator of objects of a single type, backed by a TBlockAllocator with
// and internal linked list for free items.
// Items can be constructed and returned from the pool with Take().
// The memory of items returned to the pool with Return() will be reused for
// the next calls to Take() in a LIFO order.
//
// Template parameters:
//   ItemType - the pooled item type.
//   InlineCapacity - the number of items that can be held before spilling to
//                    the heap.
template <typename ItemType, size_t InlineCapacity>
class AUTORTFM_INTERNAL TPool
{
public:
	using FItem = ItemType;

	// Constructor
	TPool() = default;

	// Acquires a new Item from the pool, forward-constructing the item with Arguments.
	template <typename... ArgumentTypes>
	FItem* Take(ArgumentTypes&&... Arguments)
	{
		void* Memory = nullptr;
		if (FreeList)
		{
			// Unlink a free entry from the free list.
			Memory = FreeList;
			FreeList = FreeList->Next;
		}
		else
		{
			// Free list is empty. Allocate another item.
			Memory = Allocator.Allocate(EntrySize, EntryAlignment);
			NumAllocated++;
		}

		AUTORTFM_ASSERT(NumInUse < NumAllocated);
		NumInUse++;

		return new (Memory) FItem(std::forward<ArgumentTypes>(Arguments)...);
	}

	// Destructs the item and returns it back to the pool so the memory can be
	// reused.
	void Return(FItem* Item)
	{
		// Destruct the item
		Item->~FItem();

		// Place the entry onto the free list
		TFreeEntry* FreeEntry = reinterpret_cast<TFreeEntry*>(Item);
		FreeEntry->Next = FreeList;
		FreeList = FreeEntry;

		AUTORTFM_ASSERT(NumInUse > 0);
		NumInUse--;
	}

	// Frees the memory allocated for the item list.
	// No items can be in use when calling.
	void Reset(bool bIgnoreNonReturned = false)
	{
		AUTORTFM_ASSERT(bIgnoreNonReturned || NumInUse == 0);
		Allocator.FreeAll();
		NumInUse = 0;
		NumAllocated = 0;
		FreeList = nullptr;
	}

	// Returns the number of items allocated.
	size_t GetNumAllocated() const
	{
		return NumAllocated;
	}

	// Returns the number of items currently in use (not returned).
	size_t GetNumInUse() const
	{
		return NumInUse;
	}

private:
	struct TFreeEntry
	{
		TFreeEntry* Next;
	};

	// The memory of each entry is used for both an Item (when in use) and a
	// TFreeEntry (when in the pool), so the size and alignment needs to be
	// the max of both.
	static constexpr size_t EntrySize = std::max<size_t>(sizeof(FItem), sizeof(TFreeEntry));
	static constexpr size_t EntryAlignment = std::max<size_t>(alignof(FItem), alignof(TFreeEntry));

	TPool(TPool&&) = delete;
	TPool(const TPool&) = delete;
	TPool& operator=(const TPool&) = delete;
	TPool& operator=(TPool&&) = delete;

	// The underlying allocator for the pool.
	TBlockAllocator<InlineCapacity * EntrySize, EntryAlignment> Allocator;
	// Free list.
	TFreeEntry* FreeList = nullptr;
	// Number of entries allocated.
	size_t NumAllocated = 0;
	// Number of entries currently in use.
	size_t NumInUse = 0;
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
