// Copyright Epic Games, Inc. All Rights Reserved.

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM/ReportHazard.h"
#include "Utils.h"

void AutoRTFM::ForTheRuntime::ReportAutoRTFMHazard(EHazardType HazardType)
{
	static constexpr const char Banner[] = R"(
             _         _        ____ _____ _____ __  __   _   _                        _
            / \  _   _| |_ ___ |  _ \_   _|  ___|  \/  | | | | | __ _ ______ _ _ __ __| |
  _____    / _ \| | | | __/ _ \| |_) || | | |_  | |\/| | | |_| |/ _` |_  / _` | '__/ _` |  _____
 |_____|  / ___ \ |_| | || (_) |  _ < | | |  _| | |  | | |  _  | (_| |/ / (_| | | | (_| | |_____|
         /_/   \_\__,_|\__\___/|_| \_\|_| |_|   |_|  |_| |_| |_|\__,_/___\__,_|_|  \__,_|

New to AutoRTFM? See Engine/Source/Runtime/AutoRTFM/Documentation/README.md for more info.
)";

	const char* Message = "An unknown error occurred.";

	switch (HazardType)
	{
		case EHazardType::CriticalSection:
			Message = R"(
A critical section was accessed while inside an AutoRTFM transaction. AutoRTFM doesn't support native
critical sections, but it does support transactionally-safe critical sections that can be used instead.

Use the call stack below to find the FCriticalSection accessed immediately before this error occurred, and
replace it with FTransactionallySafeCriticalSection. If FScopeLock is used, replace it with UE::TScopeLock.
)";
			break;

		case EHazardType::RWLock:
			Message = R"(
A read-write lock was accessed while inside an AutoRTFM transaction. AutoRTFM doesn't support native
read-write locks, but it does support transactionally-safe read-write locks that can be used instead.

Use the call stack below to find the FRWLock accessed immediately before this error occurred, and
replace it with FTransactionallySafeRWLock. If FReadScopeLock, FWriteScopeLock or FRWScopeLock are
used, replace them with UE::TReadScopeLock, UE::TWriteScopeLock or UE::TRWScopeLock.
)";
			break;

		case EHazardType::Mutex:
			Message = R"(
A mutex was accessed while inside an AutoRTFM transaction. AutoRTFM doesn't support native mutexes, but
it does support transactionally-safe mutexes that can be used instead.

Use the call stack below to find the FMutex accessed immediately before this error occurred, and
replace it with FTransactionallySafeMutex.
)";
			break;

		case EHazardType::RecursiveMutex:
			Message = R"(
A recursive mutex was accessed while inside an AutoRTFM transaction. AutoRTFM doesn't support native
recursive mutexes, but it does support transactionally-safe recursive mutexes that can be used instead.

Use the call stack below to find the FRecursiveMutex accessed immediately before this error occurred,
and replace it with FTransactionallySafeRecursiveMutex.
)";
			break;

		case EHazardType::SharedMutex:
			Message = R"(
A shared mutex was accessed while inside an AutoRTFM transaction. AutoRTFM doesn't support native
shared mutexes, but it does support transactionally-safe shared mutexes that can be used instead.

Use the call stack below to find the FSharedMutex accessed immediately before this error occurred,
and replace it with FTransactionallySafeSharedMutex.
)";
			break;

		case EHazardType::OpenFree:
			Message = R"(
Memory has been freed from the open while inside an AutoRTFM transaction. This memory was allocated
or written-to from the closed within the same transaction; freeing it makes rollback impossible.

Use the call stack below to find the location of the free call; the open block will be further up in
the stack. If possible, avoid going into the open at all; if it is unavoidable, ensure that memory
is never modified or freed from the open if it might also be written-to from the closed.
)";
			break;

		case EHazardType::SpscQueue:
			Message = R"(
An SPSC queue was accessed while inside an AutoRTFM transaction. AutoRTFM doesn't support native SPSC
queues, but it does support transactionally-safe SPSC queues that can be used instead.

Use the call stack below to find the TSpscQueue accessed immediately before this error occurred,
and replace it with TTransactionallySafeSpscQueue.
)";
			break;

		case EHazardType::MpscQueue:
			Message = R"(
An MPSC queue was accessed while inside an AutoRTFM transaction. AutoRTFM doesn't support native MPSC
queues, but it does support transactionally-safe MPSC queues that can be used instead.

Use the call stack below to find the TMpscQueue accessed immediately before this error occurred,
and replace it with TTransactionallySafeMpscQueue.
)";
			break;

		default:
			AUTORTFM_ENSURE_MSG(false, "Unrecognized hazard type %d", (int)HazardType);
			break;
	}

	AUTORTFM_WARN("%s%s", Banner, Message);
	::AutoRTFM::LogWithCallstack(autortfm_log_warn, "Call stack:");
	AUTORTFM_REPORT_ERROR("Failure was caused by an AutoRTFM hazard.");
}

#endif  // defined(__AUTORTFM) && __AUTORTFM
