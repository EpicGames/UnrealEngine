// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(__AUTORTFM) && __AUTORTFM

#include "Sanitizer.h"

#if AUTORTFM_SANITIZER

#include "AutoRTFM.h"
#include "AutoRTFM/CAPI.h"
#include "CallstackTree.h"
#include "ExternAPI.h"
#include "IntervalTree.h"
#include "StackRange.h"
#include "Utils.h"

#include <mutex>

namespace AutoRTFM::Sanitizer
{

struct AUTORTFM_INTERNAL FOSAllocator
{
	static void* Allocate(size_t Size, size_t Alignment)
	{
		void* Ptr = malloc(Size);
		AUTORTFM_ASSERT(Ptr);
		AUTORTFM_ASSERT((reinterpret_cast<uintptr_t>(Ptr) & (Alignment - 1)) == 0);
		return Ptr;
	}

	static void* Reallocate(void* Pointer, size_t Size, size_t Alignment, [[maybe_unused]] size_t PreviousSize)
	{
		void* Ptr = realloc(Pointer, Size);
		AUTORTFM_ASSERT(Ptr);
		AUTORTFM_ASSERT((reinterpret_cast<uintptr_t>(Ptr) & (Alignment - 1)) == 0);
		return Ptr;
	}

	static void* AllocateZeroed(size_t Size, size_t Alignment)
	{
		void* Ptr = calloc(1, Size);
		AUTORTFM_ASSERT(Ptr);
		AUTORTFM_ASSERT((reinterpret_cast<uintptr_t>(Ptr) & (Alignment - 1)) == 0);
		return Ptr;
	}

	static void Free(void* Ptr, [[maybe_unused]] size_t Size)
	{
		free(Ptr);
	}
};
static_assert(Allocator<FOSAllocator>);

// Maximum number of closed write entries stored in a single conflict report.
// If more are found, the report will note how many were omitted.
static constexpr size_t MaxReportedClosedWrites = 4;

struct AUTORTFM_INTERNAL FSanitizer
{
	using FCallstackTree = TCallstackTree<FOSAllocator>;

	struct FConflictingWrite
	{
		struct FClosedWriteEntry
		{
			FInterval Interval;
			FCallstackTree::FCallstack Callstack;
		};

		autortfm_log_severity Severity;
		void* Ptr;
		size_t Size;
		FCallstackTree::FCallstack OpenWriteCallstack;
		FClosedWriteEntry ClosedWrites[MaxReportedClosedWrites];
		size_t NumClosedWrites = 0;    // entries stored in ClosedWrites[]
		size_t TotalClosedWrites = 0;  // total found (may exceed MaxReportedClosedWrites)

		// Report the conflicting write, aborting the application if severity is autortfm_log_fatal.
		// As logging the conflicting write may call back into autortfm_sanitizer_open_write(), this
		// must be called without the FSanitizer lock held.
		void Report() const;
	};

	struct FLock;

	static std::atomic<Sanitizer::EMode> Mode;
	static std::atomic<size_t> NumTransactionsRunning;
	static std::atomic<bool> bRecordClosedWriteCallstacks;

	static void Initialize();
	static void Shutdown();

	FDisableHandle DisableTransaction();
	void ReenableTransaction(FDisableHandle);

	size_t NumberOfIssuesFound() const;

	void StartTransaction(TransactionID ID, FStackRange StackRange);
	void CommitTransaction(TransactionID ID);
	void AbortTransaction(TransactionID ID);

	[[nodiscard]] FConflictingWrite* OpenWrite(void* Ptr, size_t Size, void* ProgramCounter);
	void ClosedWrite(TransactionID ID, void* Ptr, size_t Size, void* ProgramCounter);

private:
	using TClosedWriteTree = TIntervalTree<FCallstackTree::FHandle, FOSAllocator>;

	struct FTransaction
	{
		TransactionID const ID;
		FTransaction* const Parent;
		FStackRange const StackRange;
		size_t DisableCount = 0;
		TClosedWriteTree ClosedWrites;
	};

	static FSanitizer* Instance;

	void PopTransaction(TransactionID ID);

	FCallstackTree::FHandle GetCallstackHandle(void* ProgramCounter);

	FTransaction* CurrentTransaction = nullptr;
	FCallstackTree CallstackTree;
	std::unordered_set<FCallstackTree::FHandle> ReportedCallstacks;
	size_t NumIssues = 0;
};

FSanitizer* FSanitizer::Instance = nullptr;
std::atomic<Sanitizer::EMode> FSanitizer::Mode{EMode::Error};
std::atomic<size_t> FSanitizer::NumTransactionsRunning{0};
std::atomic<bool> FSanitizer::bRecordClosedWriteCallstacks{true};

struct AUTORTFM_INTERNAL FSanitizer::FLock
{
	FLock()
	{
		if (!bLockedOnThisThread)
		{
			Mutex.lock();
			bLockedOnThisThread = true;
			Sanitizer = FSanitizer::Instance;
			bHoldsLock = true;
		}
	}

	~FLock()
	{
		if (bHoldsLock)
		{
			AUTORTFM_ASSERT_DEBUG(bLockedOnThisThread);
			bLockedOnThisThread = false;
			Mutex.unlock();
		}
	}

	operator bool() const
	{
		return Sanitizer;
	}

	FSanitizer* operator->() const
	{
		return Sanitizer;
	}

private:
	FLock(const FLock&) = delete;
	FLock(FLock&&) = delete;
	const FLock& operator=(const FLock&) = delete;
	FLock&& operator=(FLock&&) = delete;

	static std::mutex Mutex;
	static thread_local bool bLockedOnThisThread;

	FSanitizer* Sanitizer = nullptr;
	bool bHoldsLock = false;
};

std::mutex FSanitizer::FLock::Mutex;
thread_local bool FSanitizer::FLock::bLockedOnThisThread = false;

void FSanitizer::Initialize()
{
	Instance = New<FOSAllocator, FSanitizer>();
}

void FSanitizer::Shutdown()
{
	Delete<FOSAllocator>(Instance);
	Instance = nullptr;
}

FDisableHandle FSanitizer::DisableTransaction()
{
	if (CurrentTransaction)
	{
		CurrentTransaction->DisableCount++;
	}
	return CurrentTransaction;
}

void FSanitizer::ReenableTransaction(FDisableHandle Handle)
{
	for (FTransaction* Transaction = CurrentTransaction; Transaction; Transaction = Transaction->Parent)
	{
		if (Handle == Transaction)
		{
			AUTORTFM_ASSERT(Transaction->DisableCount > 0);
			Transaction->DisableCount--;
			break;
		}
	}
}

size_t FSanitizer::NumberOfIssuesFound() const
{
	return NumIssues;
}

void FSanitizer::StartTransaction(TransactionID ID, FStackRange StackRange)
{
	size_t DisableCount = CurrentTransaction ? CurrentTransaction->DisableCount : 0;
	CurrentTransaction = New<FOSAllocator, FTransaction>(ID, CurrentTransaction, StackRange, DisableCount);
	NumTransactionsRunning++;
}

void FSanitizer::CommitTransaction(TransactionID ID)
{
	FTransaction* const Transaction = CurrentTransaction;

	if (FTransaction* const Parent = Transaction->Parent)
	{
		for (TClosedWriteTree::FIntervalAndData Entry : Transaction->ClosedWrites)
		{
			const void* Address = reinterpret_cast<void*>(Entry.Interval.Start);
			if (!Parent->StackRange.Contains(Address))
			{
				Parent->ClosedWrites.Insert(Address, Entry.Interval.Size(), Entry.Data);
			}
		}
	}

	PopTransaction(ID);
}

void FSanitizer::AbortTransaction(TransactionID ID)
{
	PopTransaction(ID);
}

[[nodiscard]] FSanitizer::FConflictingWrite* FSanitizer::OpenWrite(void* Ptr, size_t Size, void* ProgramCounter)
{
	if (!CurrentTransaction || CurrentTransaction->DisableCount)
	{
		return nullptr;
	}

	// Fast check: does any closed write overlap the open write range?
	// This avoids constructing FClosedWriteEntry on the stack for the common no-collision case.
	bool bHasOverlap = false;
	for (FTransaction* Transaction = CurrentTransaction; Transaction && !bHasOverlap; Transaction = Transaction->Parent)
	{
		bHasOverlap = Transaction->ClosedWrites.Overlaps(Ptr, Size);
	}

	if (AUTORTFM_LIKELY(!bHasOverlap))
	{
		return nullptr;
	}

	// Collision found - gather all closed writes that overlap the open write range.
	size_t NumClosedWrites = 0;
	size_t TotalClosedWrites = 0;
	FConflictingWrite::FClosedWriteEntry ClosedWrites[MaxReportedClosedWrites];

	for (FTransaction* Transaction = CurrentTransaction; Transaction; Transaction = Transaction->Parent)
	{
		for (TClosedWriteTree::FIntervalAndData Entry : Transaction->ClosedWrites.Get(Ptr, Size))
		{
			if (NumClosedWrites < MaxReportedClosedWrites)
			{
				ClosedWrites[NumClosedWrites] = {Entry.Interval, CallstackTree.Get(Entry.Data)};
				NumClosedWrites++;
			}
			TotalClosedWrites++;
		}
	}

	const autortfm_log_severity Severity = (Mode == Sanitizer::EMode::Error) ? autortfm_log_error : autortfm_log_warn;
	FCallstackTree::FHandle const OpenHandle = GetCallstackHandle(ProgramCounter);

	if (Severity == autortfm_log_warn)
	{
		if (!ReportedCallstacks.emplace(OpenHandle).second)
		{
			return nullptr;  // Already reported for this open write site
		}
	}

	NumIssues++;

	FConflictingWrite* Result = New<FOSAllocator, FConflictingWrite>();
	Result->Severity = Severity;
	Result->Ptr = Ptr;
	Result->Size = Size;
	Result->OpenWriteCallstack = CallstackTree.Get(OpenHandle);
	Result->NumClosedWrites = NumClosedWrites;
	Result->TotalClosedWrites = TotalClosedWrites;
	for (size_t I = 0; I < NumClosedWrites; I++)
	{
		Result->ClosedWrites[I] = ClosedWrites[I];
	}
	return Result;
}

void FSanitizer::ClosedWrite(TransactionID ID, void* Ptr, size_t Size, void* ProgramCounter)
{
	FTransaction* const Transaction = CurrentTransaction;
	AUTORTFM_ASSERT(Transaction->ID == ID);

	FCallstackTree::FHandle const CallstackHandle =
		bRecordClosedWriteCallstacks ? GetCallstackHandle(ProgramCounter) : FCallstackTree::InvalidHandle;
	Transaction->ClosedWrites.Insert(Ptr, Size, CallstackHandle);
}

void FSanitizer::PopTransaction(TransactionID ID)
{
	AUTORTFM_ASSERT(ID == CurrentTransaction->ID);
	FTransaction* const Parent = CurrentTransaction->Parent;
	Delete<FOSAllocator>(CurrentTransaction);
	CurrentTransaction = Parent;
	--NumTransactionsRunning;
}

FSanitizer::FCallstackTree::FHandle FSanitizer::GetCallstackHandle(void* ProgramCounter)
{
	static constexpr size_t MaxCallstack = 128;
	autortfm_callstack_frame Callstack[MaxCallstack];
	size_t const NumCaptured = GExternAPI.CaptureCallstack(MaxCallstack, Callstack);

	// Attempt to trim off callstack frames under this function.
	size_t FramesToSkip = 0;
	for (size_t I = 0, N = std::min(NumCaptured, static_cast<size_t>(8)); I < N; I++)
	{
		if (reinterpret_cast<autortfm_callstack_frame>(ProgramCounter) == Callstack[I])
		{
			FramesToSkip += I;
			break;
		}
	}

	if (NumCaptured <= FramesToSkip)
	{
		return FCallstackTree::InvalidHandle;
	}

	return CallstackTree.Add(NumCaptured - FramesToSkip, Callstack + FramesToSkip);
}

void FSanitizer::FConflictingWrite::Report() const
{
	AUTORTFM_LOG_AT_SEVERITY(Severity, "");
	AUTORTFM_LOG_AT_SEVERITY(
		Severity, "********************************************************************************************");
	AUTORTFM_LOG_AT_SEVERITY(Severity, "AutoRTFMSan: 0x%zu bytes at %p was modified in the closed and now the open.", Size, Ptr);
	AUTORTFM_LOG_AT_SEVERITY(Severity, "This may lead to memory corruption if the transaction is aborted.");
	AUTORTFM_LOG_AT_SEVERITY(Severity, "");

	if (TotalClosedWrites == 0)
	{
		if (!bRecordClosedWriteCallstacks)
		{
			AUTORTFM_LOG_AT_SEVERITY(Severity, "AutoRTFMSan: Transactional write callstack recording is not enabled.");
			AUTORTFM_LOG_AT_SEVERITY(Severity, "AutoRTFMSan: Enable to see callstack of the last closed write.");
		}
	}
	else if (TotalClosedWrites == 1)
	{
		// Single closed write - use the familiar format.
		const FClosedWriteEntry& Entry = ClosedWrites[0];
		if (!Entry.Callstack.IsEmpty())
		{
			AUTORTFM_LOG_AT_SEVERITY(Severity, "AutoRTFMSan: The last transactional write to this address was at:");
			GExternAPI.LogCallstack(Severity, Entry.Callstack.Num(), &Entry.Callstack.Front());
			AUTORTFM_LOG_AT_SEVERITY(Severity, "");
		}
		else if (!bRecordClosedWriteCallstacks)
		{
			AUTORTFM_LOG_AT_SEVERITY(Severity, "AutoRTFMSan: Transactional write callstack recording is not enabled.");
			AUTORTFM_LOG_AT_SEVERITY(Severity, "AutoRTFMSan: Enable to see callstack of the last closed write.");
		}
	}
	else
	{
		// Multiple closed writes - numbered list.
		AUTORTFM_LOG_AT_SEVERITY(Severity, "AutoRTFMSan: %zu conflicting transactional write(s):", TotalClosedWrites);
		for (size_t I = 0; I < NumClosedWrites; I++)
		{
			const FClosedWriteEntry& Entry = ClosedWrites[I];
			AUTORTFM_LOG_AT_SEVERITY(Severity, "");
			AUTORTFM_LOG_AT_SEVERITY(
				Severity, "  [%zu] 0x%zx bytes at %p:", I + 1, Entry.Interval.Size(), reinterpret_cast<void*>(Entry.Interval.Start));
			if (!Entry.Callstack.IsEmpty())
			{
				GExternAPI.LogCallstack(Severity, Entry.Callstack.Num(), &Entry.Callstack.Front());
			}
			else if (!bRecordClosedWriteCallstacks)
			{
				AUTORTFM_LOG_AT_SEVERITY(Severity, "  <callstack recording not enabled>");
			}
		}
		if (TotalClosedWrites > MaxReportedClosedWrites)
		{
			AUTORTFM_LOG_AT_SEVERITY(Severity, "");
			AUTORTFM_LOG_AT_SEVERITY(
				Severity, "  ... and %zu more conflicting write(s).", TotalClosedWrites - MaxReportedClosedWrites);
		}
		AUTORTFM_LOG_AT_SEVERITY(Severity, "");
	}

	AUTORTFM_LOG_AT_SEVERITY(Severity, "AutoRTFMSan: The open write was at:");
	if (!OpenWriteCallstack.IsEmpty())
	{
		GExternAPI.LogCallstack(Severity, OpenWriteCallstack.Num(), &OpenWriteCallstack.Front());
	}
	else
	{
		AUTORTFM_LOG_AT_SEVERITY(Severity, "<no callstack>");
	}
	AUTORTFM_LOG_AT_SEVERITY(
		Severity, "********************************************************************************************");
	AUTORTFM_LOG_AT_SEVERITY(Severity, "");

	if (Mode == Sanitizer::EMode::Error)
	{
		AUTORTFM_FATAL("<AutoRTFMSan fatal error>");
	}
}

////////////////////////////////////////////////////////////////////////////////
// Private interface
////////////////////////////////////////////////////////////////////////////////
void Initialize()
{
	FSanitizer::Initialize();
}

void Shutdown()
{
	FSanitizer::Shutdown();
}

size_t NumberOfIssuesFound()
{
	FSanitizer::FLock Sanitizer;
	return Sanitizer->NumberOfIssuesFound();
}

void StartTransaction(TransactionID ID, FStackRange StackRange)
{
	FSanitizer::FLock Sanitizer;
	Sanitizer->StartTransaction(ID, StackRange);
}

void CommitTransaction(TransactionID ID)
{
	FSanitizer::FLock Sanitizer;
	Sanitizer->CommitTransaction(ID);
}

void AbortTransaction(TransactionID ID)
{
	FSanitizer::FLock Sanitizer;
	Sanitizer->AbortTransaction(ID);
}

void ClosedWrite(TransactionID ID, void* Ptr, size_t Size, void* ProgramCounter)
{
	FSanitizer::FLock Sanitizer;
	Sanitizer->ClosedWrite(ID, Ptr, Size, ProgramCounter);
}

////////////////////////////////////////////////////////////////////////////////
// Compiler Interface
////////////////////////////////////////////////////////////////////////////////
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_FORCENOINLINE UE_AUTORTFM_API void autortfm_sanitizer_open_write(
	void* Ptr, size_t Size) noexcept
{
	// Skip the cost of a mutex lock if there are no transactions running, or
	// if the sanitizer is disabled.
	if (AnyTransactionsInFlight() && FSanitizer::Mode != EMode::Disabled)
	{
		FSanitizer::FConflictingWrite* Err = nullptr;
		{
			FSanitizer::FLock Sanitizer;
			if (Sanitizer)
			{
				Err = Sanitizer->OpenWrite(Ptr, Size, __builtin_return_address(0));
			}
		}

		if (Err)
		{
			// Must be called outside of the sanitizer lock.
			Err->Report();
			Delete<FOSAllocator>(Err);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
// Public API
////////////////////////////////////////////////////////////////////////////////
EMode GetMode()
{
	return FSanitizer::Mode;
}

void SetMode(EMode NewMode)
{
	if (FSanitizer::Mode != NewMode)
	{
		FSanitizer::Mode = NewMode;
		if (GExternAPI.OnSanitizerModeChanged)
		{
			GExternAPI.OnSanitizerModeChanged();
		}
	}
}

bool RecordClosedWriteCallstacks()
{
	return FSanitizer::bRecordClosedWriteCallstacks;
}

void SetRecordClosedWriteCallstacks(bool bEnable)
{
	if (FSanitizer::bRecordClosedWriteCallstacks != bEnable)
	{
		FSanitizer::bRecordClosedWriteCallstacks = bEnable;
		if (GExternAPI.OnSanitizerRecordClosedWriteCallstacksChanged)
		{
			GExternAPI.OnSanitizerRecordClosedWriteCallstacksChanged();
		}
	}
}

FDisableHandle BeginDisable()
{
	if (AnyTransactionsInFlight() && FSanitizer::Mode != EMode::Disabled)
	{
		FSanitizer::FLock Sanitizer;
		if (Sanitizer)
		{
			return Sanitizer->DisableTransaction();
		}
	}
	return nullptr;
}

void EndDisable(FDisableHandle Handle)
{
	if (Handle)
	{
		FSanitizer::FLock Sanitizer;
		if (Sanitizer)
		{
			Sanitizer->ReenableTransaction(Handle);
		}
	}
}

bool AnyTransactionsInFlight()
{
	return FSanitizer::NumTransactionsRunning > 0;
}

}  // namespace AutoRTFM::Sanitizer

#endif  // AUTORTFM_SANITIZER

#endif  // defined(__AUTORTFM) && __AUTORTFM
