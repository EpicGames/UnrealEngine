// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(__AUTORTFM) && __AUTORTFM

#include "AutoRTFM.h"

#if AUTORTFM_SANITIZER

#include "StackRange.h"

namespace AutoRTFM::Sanitizer
{

#define AUTORTFM_SANITIZER_INTERNAL [[clang::autortfm(autortfm_mode_internal)]]

AUTORTFM_SANITIZER_INTERNAL void Initialize();
AUTORTFM_SANITIZER_INTERNAL void Shutdown();

AUTORTFM_SANITIZER_INTERNAL size_t NumberOfIssuesFound();

AUTORTFM_SANITIZER_INTERNAL void StartTransaction(TransactionID ID, FStackRange StackRange);
AUTORTFM_SANITIZER_INTERNAL void CommitTransaction(TransactionID ID);
AUTORTFM_SANITIZER_INTERNAL void AbortTransaction(TransactionID ID);

AUTORTFM_SANITIZER_INTERNAL void ClosedWrite(TransactionID ID, void* Ptr, size_t Size, void* ProgramCounter);

#undef AUTORTFM_SANITIZER_INTERNAL

}  // namespace AutoRTFM::Sanitizer

#endif  //  AUTORTFM_SANITIZER
#endif  // defined(__AUTORTFM) && __AUTORTFM
