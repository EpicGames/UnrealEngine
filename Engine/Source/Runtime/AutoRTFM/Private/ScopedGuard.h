// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Utils.h"

namespace AutoRTFM
{

template <typename T>
class AUTORTFM_INTERNAL TScopedGuard
{
public:
	TScopedGuard(T& Ref, const T& Value) : OldValue(Ref), Ref(Ref)
	{
		Ref = Value;
	}

	~TScopedGuard()
	{
		Ref = OldValue;
	}

private:
	T OldValue;
	T& Ref;
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
