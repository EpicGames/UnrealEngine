// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Allocator.h"
#include "ContainerValidation.h"
#include "Utils.h"

#include <cstddef>
#include <initializer_list>
#include <type_traits>

namespace AutoRTFM
{

// A stack container with an inline fixed capacity that once exceeded spills to
// the heap.
//
// Notes:
//  * Heap allocated memory is not automatically freed when popping elements.
//    Only calling Reset() or destructing the stack will free heap allocated
//    memory.
//  * This class is not relocatable and so is not safe to use in UE's containers
//    which require elements to be relocatable.
//
// Template parameters:
//   T - the element type. Must support copy and move constructors.
//   InlineCapacity - the number of elements that can be held before spilling to
//                    the heap.
//   Validation - if Disabled, do not perform validity assertions.
template <typename T, size_t InlineCapacity, Allocator Allocator = FAllocator,
	EContainerValidation Validation = EContainerValidation::Enabled>
class AUTORTFM_INTERNAL TStack
{
public:
	using ElementType = T;

	// Constructor.
	TStack() = default;

	// Constructor from initializer list.
	TStack(std::initializer_list<T> List)
	{
		Reserve(List.size());
		for (const T& Item : List)
		{
			Push(Item);
		}
	}

	// Copy constructor.
	TStack(const TStack& Other)
	{
		CopyFrom(Other);
	}

	// Move constructor.
	TStack(TStack&& Other)
	{
		MoveFrom(std::move(Other));
	}

	// Destructor.
	// Frees all the memory allocated by the allocator.
	~TStack()
	{
		Reset();
	}

	// Copy assignment operator
	TStack& operator=(const TStack& Other)
	{
		if (&Other != this)
		{
			Clear();
			CopyFrom(Other);
		}
		return *this;
	}

	// Move assignment operator
	TStack& operator=(TStack&& Other)
	{
		if (&Other != this)
		{
			Reset();
			MoveFrom(std::move(Other));
		}
		return *this;
	}

	// Clears all the items from the stack, preserving the capacity.
	void Clear()
	{
		for (T& Item : *this)
		{
			Item.~T();
		}
		Count = 0;
	}

	// Clears all the items from the stack, freeing all heap allocations and
	// resetting the capacity to InlineCapacity
	void Reset()
	{
		Clear();
		if (Data != InlineData())
		{
			Free();
			Data = reinterpret_cast<T*>(InlineDataBuffer);
		}
		Capacity = InlineCapacity;
	}

	// Pushes a new item on to the stack.
	inline void Push(const T& Item)
	{
		if (Count >= Capacity)
		{
			Reserve(Capacity * 2);
		}

		new (Data + Count) T(Item);
		++Count;
	}

	// Moves all the items from Other to this stack.
	// Other will hold no elements after calling.
	inline void PushAll(TStack&& Other)
	{
		size_t NewCount = Count + Other.Count;
		if (NewCount >= Capacity)
		{
			Reserve(std::max<size_t>(NewCount, Capacity * 2));
		}

		if constexpr (std::is_trivial_v<T>)
		{
			memcpy(Data + Count, Other.Data, Other.Count * sizeof(T));
		}
		else
		{
			for (size_t I = 0, N = Other.Count; I < N; I++)
			{
				new (Data + Count + I) T(std::move(Other.Data[I]));
				Other.Data[I].~T();
			}
		}
		Count = NewCount;
		if (Other.Data != Other.InlineData())
		{
			Other.Free();
		}
		Other.SetToInitialState();
	}

	// Removes the last item on the stack.
	inline void Pop()
	{
		AUTORTFM_ASSERT(Validation == EContainerValidation::Disabled || Count > 0);
		--Count;
		Data[Count].~T();
	}

	// Reserves memory for at NewCapacity items.
	inline void Reserve(size_t NewCapacity)
	{
		if (NewCapacity <= Capacity)
		{
			return;  // Already has space for the new capacity
		}

		if constexpr (std::is_trivial_v<T>)
		{
			if (Data == InlineData())
			{
				T* NewData = reinterpret_cast<T*>(Allocator::Allocate(NewCapacity * sizeof(T), alignof(T)));
				memcpy(NewData, InlineDataBuffer, Count * sizeof(T));
				Data = NewData;
			}
			else
			{
				size_t const PreviousSize = Capacity * sizeof(T);
				Data = reinterpret_cast<T*>(Allocator::Reallocate(Data, NewCapacity * sizeof(T), alignof(T), PreviousSize));
			}
		}
		else
		{
			T* NewData = reinterpret_cast<T*>(Allocator::Allocate(NewCapacity * sizeof(T), alignof(T)));
			for (size_t I = 0, N = Count; I < N; I++)
			{
				new (NewData + I) T(std::move(Data[I]));
				Data[I].~T();
			}
			if (Data != InlineData())
			{
				Free();
			}
			Data = NewData;
		}

		Capacity = NewCapacity;
	}

	inline size_t Num() const
	{
		return Count;
	}

	inline bool IsEmpty() const
	{
		return Count == 0;
	}

	T& operator[](size_t Index)
	{
		AUTORTFM_ASSERT(Validation == EContainerValidation::Disabled || Index < Count);
		return Data[Index];
	}

	const T& operator[](size_t Index) const
	{
		AUTORTFM_ASSERT(Validation == EContainerValidation::Disabled || Index < Count);
		return Data[Index];
	}

	T& Front()
	{
		AUTORTFM_ASSERT(Validation == EContainerValidation::Disabled || Count > 0);
		return Data[0];
	}

	const T& Front() const
	{
		AUTORTFM_ASSERT(Validation == EContainerValidation::Disabled || Count > 0);
		return Data[0];
	}

	T& Back()
	{
		AUTORTFM_ASSERT(Validation == EContainerValidation::Disabled || Count > 0);
		return Data[Count - 1];
	}

	const T& Back() const
	{
		AUTORTFM_ASSERT(Validation == EContainerValidation::Disabled || Count > 0);
		return Data[Count - 1];
	}

	T* begin()
	{
		return Data;
	}
	const T* begin() const
	{
		return Data;
	}
	T* end()
	{
		return Data + Count;
	}
	const T* end() const
	{
		return Data + Count;
	}

	// Equality operator
	template <typename OtherT, size_t OtherInlineCapacity, typename OtherAllocator>
	bool operator==(const TStack<OtherT, OtherInlineCapacity, OtherAllocator>& Other) const
	{
		if (Num() != Other.Num())
		{
			return false;
		}
		for (size_t I = 0, N = Num(); I < N; I++)
		{
			if (Data[I] != Other[I])
			{
				return false;
			}
		}
		return true;
	}

private:
	// Frees the heap allocation for this TStack.
	// Note: Does not null Data or reset Capacity.
	void Free()
	{
		AUTORTFM_ASSERT(Data != InlineData());
		size_t const Size = Capacity * sizeof(T);
		Allocator::Free(Data, Size);
	}

	// Copies the data from Other to this stack.
	// This stack must be empty before calling.
	void CopyFrom(const TStack& Other)
	{
		AUTORTFM_ASSERT(IsEmpty());
		Reserve(Other.Count);
		if constexpr (std::is_trivial_v<T>)
		{
			memcpy(Data, Other.Data, Other.Count * sizeof(T));
		}
		else
		{
			for (size_t I = 0, N = Other.Count; I < N; I++)
			{
				new (Data + I) T(Other.Data[I]);
			}
		}
		Count = Other.Count;
	}

	// Moves the data from Other to this stack.
	// This stack must be empty before calling.
	void MoveFrom(TStack&& Other)
	{
		AUTORTFM_ASSERT(IsEmpty());
		if (Other.Data == Other.InlineData())
		{
			if constexpr (std::is_trivial_v<T>)
			{
				memcpy(Data, Other.Data, Other.Count * sizeof(T));
			}
			else
			{
				for (size_t I = 0, N = Other.Count; I < N; I++)
				{
					new (Data + I) T(std::move(Other.Data[I]));
					Other.Data[I].~T();
				}
			}
		}
		else
		{
			// Steal the heap allocation from Other.
			Data = Other.Data;
			Capacity = Other.Capacity;
		}
		Count = Other.Count;
		Other.SetToInitialState();
	}

	// Sets Data, Capacity and Count to the initially default-constructed values.
	// Warning: This does not free or destruct any elements held by this stack.
	void SetToInitialState()
	{
		Data = InlineData();
		Capacity = InlineCapacity;
		Count = 0;
	}

	T* InlineData()
	{
		return reinterpret_cast<T*>(InlineDataBuffer);
	}
	const T* InlineData() const
	{
		return reinterpret_cast<const T*>(InlineDataBuffer);
	}

	alignas(T) std::byte InlineDataBuffer[sizeof(T) * InlineCapacity];
	T* Data = reinterpret_cast<T*>(InlineDataBuffer);
	size_t Capacity = InlineCapacity;
	size_t Count = 0;
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
