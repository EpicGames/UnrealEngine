// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

namespace AutoRTFM
{

// FStackRange represents a stack memory range.
// It is assumed the stack grows downwards.
struct FStackRange
{
	void* Low = nullptr;   // One byte past the end of the stack range.
	void* High = nullptr;  // The first byte of the stack range.

	// Returns true if the stack range contains Address
	bool Contains(const void* Address) const
	{
		return Address > Low && Address <= High;
	}

	// Equality operator
	bool operator==(const FStackRange& Other) const
	{
		return Low == Other.Low && High == Other.High;
	}
	// In-equality operator
	bool operator!=(const FStackRange& Other) const
	{
		return !(*this == Other);
	}
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
