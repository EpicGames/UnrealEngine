// Copyright Epic Games, Inc. All Rights Reserved.

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM.h"
#include "BuildMacros.h"
#include "ContextInlines.h"
#include "Memcpy.h"
#include "Utils.h"

#include <algorithm>
#include <charconv>
#include <cstddef>
#include <float.h>
#include <functional>  // note: introduces additional math overloads
#include <stdarg.h>
#include <stdio.h>
#include <stdlib.h>
#include <string.h>
#include <wchar.h>

#ifdef __APPLE__
#include <malloc/malloc.h>
#else
#include <malloc.h>
#endif

#if AUTORTFM_PLATFORM_WINDOWS
#include "WindowsHeader.h"
#endif

#if AUTORTFM_PLATFORM_LINUX
#include <sys/stat.h>
#include <sys/time.h>
#include <sys/types.h>
#include <unistd.h>
#endif

#if __has_include(<sanitizer/asan_interface.h>)
#include <sanitizer/asan_interface.h>
#if defined(__SANITIZE_ADDRESS__)
#define AUTORTFM_ASAN_ENABLED 1
#elif defined(__has_feature)
#if __has_feature(address_sanitizer)
#define AUTORTFM_ASAN_ENABLED 1
#endif
#endif
#endif

#ifndef AUTORTFM_ASAN_ENABLED
#define AUTORTFM_ASAN_ENABLED 0
#endif

#ifdef _MSC_VER
// BEGIN: Disable warning about deprecated STD C functions.
#pragma warning(push)
#pragma warning(disable : 4996)
#endif

namespace AutoRTFM
{

namespace
{

// A helper that opens a FILE to "/dev/null" on first call to Get()
// and automatically closes the file on static destruction.
class AUTORTFM_INTERNAL FNullFile
{
public:
	static FILE* Get()
	{
		static FNullFile Instance;
		return Instance.File;
	}

private:
	FNullFile() : File(fopen("/dev/null", "wb")) {}
	~FNullFile()
	{
		fclose(File);
	}
	FILE* const File;
};

template <typename T>
inline void RecordOpenWriteIfNotNull(T* Ptr)
{
	if (Ptr != nullptr)
	{
		RecordOpenWrite(Ptr);
	}
}

AUTORTFM_INTERNAL void ThrowErrorFormatContainsPercentN()
{
	AUTORTFM_WARN("AutoRTFM does not support format strings containing '%%n'");
	FContext* Context = FContext::Get();
	Context->AbortTransactionAndThrow(ETransactionStatus::AbortedByLanguage);
}

// Throws an error if the format string contains a '%n'.
AUTORTFM_INTERNAL static void ThrowIfFormatContainsPercentN(const char* Format)
{
	for (const char* P = Format; *P != '\0'; ++P)
	{
		if (*P == '%')
		{
			switch (*++P)
			{
				case 'n':
					ThrowErrorFormatContainsPercentN();
					break;
				case '\0':
					return;
			}
		}
	}
}

// Throws an error if the format string contains a '%n'.
AUTORTFM_INTERNAL static void ThrowIfFormatContainsPercentN(const wchar_t* Format)
{
	for (const wchar_t* P = Format; *P != L'\0'; ++P)
	{
		if (*P == L'%')
		{
			switch (*++P)
			{
				case L'n':
					ThrowErrorFormatContainsPercentN();
					break;
				case L'\0':
					return;
			}
		}
	}
}

}  // anonymous namespace

#if AUTORTFM_PLATFORM_WINDOWS
// vvvvvvvvvvvvvvvvvvvvvvvvvvvvvv AUTORTFM_PLATFORM_WINDOWS vvvvvvvvvvvvvvvvvvvvvvvvvvvvv
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API long long RTFM__strtoi64(const char* String, char** EndPtr, int Radix)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _strtoi64(String, EndPtr, Radix);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API long long RTFM__wcstoi64(const wchar_t* String, wchar_t** EndPtr, int Radix)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _wcstoi64(String, EndPtr, Radix);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API unsigned long long RTFM__wcstoui64(const wchar_t* String, wchar_t** EndPtr, int Radix)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _wcstoui64(String, EndPtr, Radix);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API double RTFM__wcstod_l(const wchar_t* String, wchar_t** EndPtr, _locale_t Locale)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _wcstod_l(String, EndPtr, Locale);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API float RTFM__wcstof_l(const wchar_t* String, wchar_t** EndPtr, _locale_t Locale)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _wcstof_l(String, EndPtr, Locale);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API long double RTFM__wcstold_l(const wchar_t* String, wchar_t** EndPtr, _locale_t Locale)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _wcstold_l(String, EndPtr, Locale);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API long RTFM__wcstol_l(
	const wchar_t* String, wchar_t** EndPtr, int Radix, _locale_t Locale)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _wcstol_l(String, EndPtr, Radix, Locale);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API long long RTFM__wcstoll_l(
	const wchar_t* String, wchar_t** EndPtr, int Radix, _locale_t Locale)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _wcstoll_l(String, EndPtr, Radix, Locale);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API unsigned long RTFM__wcstoul_l(
	const wchar_t* String, wchar_t** EndPtr, int Radix, _locale_t Locale)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _wcstoul_l(String, EndPtr, Radix, Locale);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API unsigned long long RTFM__wcstoull_l(
	const wchar_t* String, wchar_t** EndPtr, int Radix, _locale_t Locale)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _wcstoull_l(String, EndPtr, Radix, Locale);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API long long RTFM__wcstoi64_l(
	const wchar_t* String, wchar_t** EndPtr, int Radix, _locale_t Locale)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _wcstoi64_l(String, EndPtr, Radix, Locale);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API unsigned long long RTFM__wcstoui64_l(
	const wchar_t* String, wchar_t** EndPtr, int Radix, _locale_t Locale)
{
	RecordOpenWriteIfNotNull(EndPtr);
	return _wcstoui64_l(String, EndPtr, Radix, Locale);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API FILE* RTFM___acrt_iob_func(unsigned Index)
{
	switch (Index)
	{
		case 1:
		case 2:
			return __acrt_iob_func(Index);
		default:
		{
			AUTORTFM_WARN("Attempt to get file descriptor %d (not 1 or 2) in __acrt_iob_func.", Index);
			FContext* Context = FContext::Get();
			Context->AbortTransactionAndThrow(ETransactionStatus::AbortedByLanguage);
			return NULL;
		}
	}
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM___std_swap_ranges_trivially_swappable_noalias(
	void* FirstA, void* LastA, void* FirstB) noexcept
{
	if (AUTORTFM_LIKELY(LastA > FirstA))
	{
		std::byte* PtrA = reinterpret_cast<std::byte*>(FirstA);
		std::byte* PtrB = reinterpret_cast<std::byte*>(FirstB);
		size_t const Size = reinterpret_cast<std::byte*>(LastA) - PtrA;
		FContext* Context = FContext::Get();
		Context->RecordWrite(PtrA, Size);
		Context->RecordWrite(PtrB, Size);
		for (size_t I = 0; I < Size; I++, PtrA++, PtrB++)
		{
			std::byte Tmp = *PtrA;
			*PtrA = *PtrB;
			*PtrB = Tmp;
		}
	}
}

// FIXME: Does not currently support %n format specifiers.
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM___stdio_common_vfprintf(
	unsigned __int64 Options, FILE* Stream, char const* Format, _locale_t Locale, va_list ArgList)
{
	ThrowIfFormatContainsPercentN(Format);

	return __stdio_common_vfprintf(Options, Stream, Format, Locale, ArgList);
}

// FIXME: Does not currently support %n format specifiers.
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM___stdio_common_vsprintf(
	unsigned __int64 Options, char* Buffer, size_t BufferCount, char const* Format, _locale_t Locale, va_list ArgList)
{
	ThrowIfFormatContainsPercentN(Format);

	if (nullptr != Buffer && 0 != BufferCount)
	{
		va_list ArgList2;
		va_copy(ArgList2, ArgList);
		int Count = __stdio_common_vsprintf(Options, nullptr, 0, Format, Locale, ArgList2);
		va_end(ArgList2);

		if (Count >= 0)
		{
			size_t NumBytes = std::min(BufferCount, static_cast<size_t>(1 + Count)) * sizeof(char);
			FContext* Context = FContext::Get();
			Context->RecordWrite(Buffer, NumBytes);
		}
	}

	return __stdio_common_vsprintf(Options, Buffer, BufferCount, Format, Locale, ArgList);
}

// FIXME: Does not currently support %n format specifiers.
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM___stdio_common_vswprintf(
	unsigned __int64 Options, wchar_t* Buffer, size_t BufferCount, wchar_t const* Format, _locale_t Locale, va_list ArgList)
{
	ThrowIfFormatContainsPercentN(Format);

	if (nullptr != Buffer && 0 != BufferCount)
	{
		va_list ArgList2;
		va_copy(ArgList2, ArgList);
		int Count = __stdio_common_vswprintf(Options, nullptr, 0, Format, Locale, ArgList2);
		va_end(ArgList2);

		if (Count >= 0)
		{
			size_t NumBytes = std::min(BufferCount, static_cast<size_t>(1 + Count)) * sizeof(wchar_t);
			FContext* Context = FContext::Get();
			Context->RecordWrite(Buffer, NumBytes);
		}
	}

	return __stdio_common_vswprintf(Options, Buffer, BufferCount, Format, Locale, ArgList);
}

// FIXME: Does not currently support %n format specifiers.
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM___stdio_common_vfwprintf(
	unsigned __int64 Options, FILE* Stream, wchar_t const* Format, _locale_t Locale, va_list ArgList)
{
	ThrowIfFormatContainsPercentN(Format);

	return __stdio_common_vfwprintf(Options, Stream, Format, Locale, ArgList);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API BOOL RTFM_TlsSetValue(DWORD dwTlsIndex, LPVOID lpTlsValue)
{
	LPVOID OriginalValue = TlsGetValue(dwTlsIndex);

	AutoRTFM::ForTheRuntime::RegisterOnAbortFromTheOpen([dwTlsIndex, OriginalValue] { TlsSetValue(dwTlsIndex, OriginalValue); });

	return TlsSetValue(dwTlsIndex, lpTlsValue);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API BOOL RTFM_FlsSetValue(DWORD dwFlsIndex, LPVOID lpFlsValue)
{
	LPVOID OriginalValue = FlsGetValue(dwFlsIndex);

	AutoRTFM::ForTheRuntime::RegisterOnAbortFromTheOpen([dwFlsIndex, OriginalValue] { FlsSetValue(dwFlsIndex, OriginalValue); });

	return FlsSetValue(dwFlsIndex, lpFlsValue);
}

// ^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^ AUTORTFM_PLATFORM_WINDOWS ^^^^^^^^^^^^^^^^^^^^^^^^^^^^^
#else
// vvvvvvvvvvvvvvvvvvvvvvvvvvvvv !AUTORTFM_PLATFORM_WINDOWS vvvvvvvvvvvvvvvvvvvvvvvvvvvvv
extern "C" size_t _ZNSt3__112__next_primeEm(size_t N) __attribute__((weak));
// ^^^^^^^^^^^^^^^^^^^^^^^^^^^^^ !AUTORTFM_PLATFORM_WINDOWS ^^^^^^^^^^^^^^^^^^^^^^^^^^^^^
#endif

#if AUTORTFM_PLATFORM_LINUX
// vvvvvvvvvvvvvvvvvvvvvvvvvvvvvv AUTORTFM_PLATFORM_LINUX vvvvvvvvvvvvvvvvvvvvvvvvvvvvvv
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM_stat(const char* Path, struct stat* StatBuf) throw()
{
	FContext* Context = FContext::Get();
	Context->RecordWrite(StatBuf, sizeof(*StatBuf));

	return stat(Path, StatBuf);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM_fstat(int Fd, struct stat* StatBuf) throw()
{
	FContext* Context = FContext::Get();
	Context->RecordWrite(StatBuf, sizeof(*StatBuf));

	return fstat(Fd, StatBuf);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM___xstat(int Ver, const char* Path, struct stat* StatBuf) throw()
{
	FContext* Context = FContext::Get();
	Context->RecordWrite(StatBuf, sizeof(*StatBuf));

	return __xstat(Ver, Path, StatBuf);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM___fxstat(int Ver, int Fd, struct stat* StatBuf) throw()
{
	FContext* Context = FContext::Get();
	Context->RecordWrite(StatBuf, sizeof(*StatBuf));

	return __fxstat(Ver, Fd, StatBuf);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API float RTFM_strtof32(const char* String, char** EndPtr) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return strtof32(String, EndPtr);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API double RTFM_strtof64(const char* String, char** EndPtr) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return strtof64(String, EndPtr);
}

// ^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^ AUTORTFM_PLATFORM_LINUX ^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^
#endif

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_memcpy(void* Dst, const void* Src, size_t Size) throw()
{
	return Memcpy(Dst, Src, Size, FContext::Get());
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_memmove(void* Dst, const void* Src, size_t Size) throw()
{
	return Memmove(Dst, Src, Size, FContext::Get());
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_memset(void* Dst, int Value, size_t Size) throw()
{
	return Memset(Dst, Value, Size, FContext::Get());
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_malloc(size_t Size) throw()
{
	void* Result = malloc(Size);
	FContext* Context = FContext::Get();
	Context->GetCurrentTransaction()->DeferUntilAbort([Result] { free(Result); });
	Context->DidAllocate(Result, Size);
	return Result;
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_calloc(size_t Count, size_t Size) throw()
{
	void* Result = calloc(Count, Size);
	FContext* Context = FContext::Get();
	Context->GetCurrentTransaction()->DeferUntilAbort([Result] { free(Result); });
	Context->DidAllocate(Result, Count * Size);
	return Result;
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_free(void* Ptr) throw()
{
	if (Ptr)
	{
		FContext* Context = FContext::Get();
		Context->GetCurrentTransaction()->DeferUntilCommit([Ptr] { free(Ptr); });
	}
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_realloc(void* Ptr, size_t Size) throw()
{
	void* NewObject = RTFM_malloc(Size);
	if (Ptr)
	{
#if defined(__APPLE__)
		const size_t OldSize = malloc_size(Ptr);
#elif defined(_WIN32)
		const size_t OldSize = _msize(Ptr);
#else
		const size_t OldSize = malloc_usable_size(Ptr);
#endif
		FContext* Context = FContext::Get();
		MemcpyToNew(NewObject, Ptr, std::min(OldSize, Size), Context);
		RTFM_free(Ptr);
	}
	return NewObject;
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API char* RTFM_strcpy(char* const Dst, const char* const Src) throw()
{
	const size_t SrcLen = strlen(Src);

	FContext* Context = FContext::Get();
	Context->RecordWrite(Dst, SrcLen + sizeof(char));
	return strcpy(Dst, Src);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API char* RTFM_strncpy(char* const Dst, const char* const Src, const size_t Num) throw()
{
	FContext* Context = FContext::Get();
	Context->RecordWrite(Dst, Num);
	return strncpy(Dst, Src, Num);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API char* RTFM_strcat(char* const Dst, const char* const Src) throw()
{
	const size_t DstLen = strlen(Dst);
	const size_t SrcLen = strlen(Src);

	FContext* Context = FContext::Get();
	Context->RecordWrite(Dst + DstLen, SrcLen + 1);
	return strcat(Dst, Src);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API char* RTFM_strncat(char* const Dst, const char* const Src, const size_t Num) throw()
{
	const size_t DstLen = strlen(Dst);

	FContext* Context = FContext::Get();
	Context->RecordWrite(Dst + DstLen, Num + 1);
	return strncat(Dst, Src, Num);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API long RTFM_strtol(const char* String, char** EndPtr, int Radix) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return strtol(String, EndPtr, Radix);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API long long RTFM_strtoll(const char* String, char** EndPtr, int Radix) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return strtoll(String, EndPtr, Radix);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API unsigned long RTFM_strtoul(const char* String, char** EndPtr, int Radix) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return strtoul(String, EndPtr, Radix);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API unsigned long long RTFM_strtoull(const char* String, char** EndPtr, int Radix) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return strtoull(String, EndPtr, Radix);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API float RTFM_strtof(const char* String, char** EndPtr) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return strtof(String, EndPtr);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API double RTFM_strtod(const char* String, char** EndPtr) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return strtod(String, EndPtr);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API double RTFM_wcstod(const wchar_t* String, wchar_t** EndPtr) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return wcstod(String, EndPtr);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API float RTFM_wcstof(const wchar_t* String, wchar_t** EndPtr) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return wcstof(String, EndPtr);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API long double RTFM_wcstold(const wchar_t* String, wchar_t** EndPtr) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return wcstold(String, EndPtr);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API long RTFM_wcstol(const wchar_t* String, wchar_t** EndPtr, int Radix) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return wcstol(String, EndPtr, Radix);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API long long RTFM_wcstoll(const wchar_t* String, wchar_t** EndPtr, int Radix) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return wcstoll(String, EndPtr, Radix);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API unsigned long RTFM_wcstoul(const wchar_t* String, wchar_t** EndPtr, int Radix) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return wcstoul(String, EndPtr, Radix);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API unsigned long long RTFM_wcstoull(
	const wchar_t* String, wchar_t** EndPtr, int Radix) throw()
{
	RecordOpenWriteIfNotNull(EndPtr);
	return wcstoull(String, EndPtr, Radix);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_i(char* First, char* Last, int Value, int Base)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Base);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_u(
	char* First, char* Last, unsigned int Value, int Base)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Base);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_i8(char* First, char* Last, int8_t Value, int Base)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Base);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_u8(char* First, char* Last, uint8_t Value, int Base)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Base);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_i16(char* First, char* Last, int16_t Value, int Base)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Base);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_u16(
	char* First, char* Last, uint16_t Value, int Base)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Base);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_i32(char* First, char* Last, int32_t Value, int Base)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Base);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_u32(
	char* First, char* Last, uint32_t Value, int Base)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Base);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_i64(char* First, char* Last, int64_t Value, int Base)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Base);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_u64(
	char* First, char* Last, uint64_t Value, int Base)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Base);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_f(char* First, char* Last, float Value)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_f_F(
	char* First, char* Last, float Value, std::chars_format Format)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Format);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_f_FP(
	char* First, char* Last, float Value, std::chars_format Format, int Precision)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Format, Precision);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_d(char* First, char* Last, double Value)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_d_F(
	char* First, char* Last, double Value, std::chars_format Format)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Format);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_d_FP(
	char* First, char* Last, double Value, std::chars_format Format, int Precision)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Format, Precision);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_ld(char* First, char* Last, long double Value)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_ld_F(
	char* First, char* Last, long double Value, std::chars_format Format)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Format);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API std::to_chars_result RTFM__ToChars_ld_FP(
	char* First, char* Last, long double Value, std::chars_format Format, int Precision)
{
	RecordOpenWrite(First, Last - First);
	return std::to_chars(First, Last, Value, Format, Precision);
}

// FIXME: Does not currently support %n format specifiers.
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM_vsnprintf(char* Str, size_t Size, const char* Format, va_list ArgList) throw()
{
	ThrowIfFormatContainsPercentN(Format);

	if (nullptr != Str && 0 != Size)
	{
		va_list ArgList2;
		va_copy(ArgList2, ArgList);
		int Count = vsnprintf(nullptr, 0, Format, ArgList2);
		va_end(ArgList2);

		if (Count >= 0)
		{
			size_t NumBytes = std::min(Size, static_cast<size_t>(1 + Count)) * sizeof(char);
			FContext* Context = FContext::Get();
			Context->RecordWrite(Str, NumBytes);
		}
	}

	return vsnprintf(Str, Size, Format, ArgList);
}

// FIXME: Does not currently support %n format specifiers.
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM_vswprintf(wchar_t* Str, size_t Size, const wchar_t* Format, va_list ArgList)
{
	ThrowIfFormatContainsPercentN(Format);

	if (nullptr != Str && 0 != Size)
	{
		va_list ArgList2;
		va_copy(ArgList2, ArgList);

#if AUTORTFM_PLATFORM_WINDOWS
		int Count = vswprintf(nullptr, 0, Format, ArgList2);
#else
		// vswprintf(nullptr, 0, ...) will return -1.
		int Count = vfwprintf(FNullFile::Get(), Format, ArgList2);
#endif

		va_end(ArgList2);

		size_t NumChars = std::min(Size, static_cast<size_t>(1 + std::max(Count, 0)));
		size_t NumBytes = NumChars * sizeof(wchar_t);
		if (NumBytes >= 0)
		{
			FContext* Context = FContext::Get();
			Context->RecordWrite(Str, NumBytes);
		}
	}

	return vswprintf(Str, Size, Format, ArgList);
}

// FIXME: Does not currently support %n format specifiers.
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM_swprintf(wchar_t* Buffer, size_t BufferCount, wchar_t const* Format, ...)
{
	va_list ArgList;

	va_start(ArgList, Format);
	int Count = RTFM_vswprintf(Buffer, BufferCount, Format, ArgList);
	va_end(ArgList);

	return Count;
}

// FIXME: Does not currently support %n format specifiers.
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM_snprintf(char* Str, size_t Size, const char* Format, ...) throw()
{
	va_list ArgList;

	va_start(ArgList, Format);
	int Count = RTFM_vsnprintf(Str, Size, Format, ArgList);
	va_end(ArgList);

	return Count;
}

// FIXME: Does not currently support %n format specifiers.
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM_printf(const char* Format, ...)
{
	ThrowIfFormatContainsPercentN(Format);

	va_list ArgList;
	va_start(ArgList, Format);
	int Result = vprintf(Format, ArgList);
	va_end(ArgList);
	return Result;
}

// FIXME: Does not currently support %n format specifiers.
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM_wprintf(const wchar_t* Format, ...)
{
	ThrowIfFormatContainsPercentN(Format);

	va_list ArgList;
	va_start(ArgList, Format);
	int Result = vwprintf(Format, ArgList);
	va_end(ArgList);
	return Result;
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API wchar_t* RTFM_wcscpy(wchar_t* Dst, const wchar_t* Src) throw()
{
	const size_t SrcLen = wcslen(Src);

	FContext* Context = FContext::Get();
	Context->RecordWrite(Dst, (SrcLen + 1) * sizeof(wchar_t));
	return wcscpy(Dst, Src);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API wchar_t* RTFM_wcsncpy(wchar_t* Dst, const wchar_t* Src, size_t Count) throw()
{
	FContext* Context = FContext::Get();
	Context->RecordWrite(Dst, Count * sizeof(wchar_t));
	return wcsncpy(Dst, Src, Count);
}

extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API int RTFM_atexit(void (*Callback)(void)) throw()
{
	FContext* Context = FContext::Get();
	Context->GetCurrentTransaction()->DeferUntilCommit([Callback] { atexit(Callback); });

	return 0;
}

#if AUTORTFM_PLATFORM_WINDOWS && AUTORTFM_EXCEPTIONS_ENABLED
extern "C" void __stdcall _CxxThrowException(void* pExceptionObject, struct _ThrowInfo* pThrowInfo);
#endif

#if AUTORTFM_PLATFORM_LINUX && AUTORTFM_EXCEPTIONS_ENABLED
extern "C" void __cxa_throw(void* thrown_exception, struct std::type_info* tinfo, void (*dest)(void*));
extern "C" void* __cxa_allocate_exception(size_t thrown_size) throw();
extern "C" void* __cxa_begin_catch(void* exceptionObject) throw();
extern "C" void __cxa_end_catch();
#endif

template <typename OpenNewFn, typename OpenDeleteFn, typename... ArgTys>
UE_AUTORTFM_FORCEINLINE static void* RTFM_ClosedNew(OpenNewFn* New, OpenDeleteFn* Delete, size_t Size, ArgTys... Args)
{
	void* Result = New(Size, Args...);
	FContext* Context = FContext::Get();
	Context->GetCurrentTransaction()->DeferUntilAbort([=] { Delete(Result, Args...); });
	Context->DidAllocate(Result, Size);
	return Result;
}

template <typename OpenDeleteFn, typename... ArgTys>
UE_AUTORTFM_FORCEINLINE static void RTFM_ClosedDelete(OpenDeleteFn* Delete, void* Pointer, ArgTys... Args)
{
	if (Pointer)
	{
		FContext* Context = FContext::Get();
		Context->GetCurrentTransaction()->DeferUntilCommit([=] { Delete(Pointer, Args...); });
	}
}

// operator new(size_t)
// ??2@YAPEAX_K@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_Znwm(size_t Size)
{
	void* (*New)(size_t) = &operator new;
	void (*Delete)(void*) = &operator delete;
	return RTFM_ClosedNew(New, Delete, Size);
}

// operator new[](size_t)
// ??_U@YAPEAX_K@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_Znam(size_t Size)
{
	void* (*New)(size_t) = &operator new[];
	void (*Delete)(void*) = &operator delete[];
	return RTFM_ClosedNew(New, Delete, Size);
}

// operator new(size_t, std::nothrow_t const&)
// ??2@YAPEAX_KAEBUnothrow_t@std@@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_ZnwmRKSt9nothrow_t(size_t Size, std::nothrow_t const& NoThrow)
{
	void* (*New)(size_t, std::nothrow_t const&) = &operator new;
	void (*Delete)(void*, std::nothrow_t const&) = &operator delete;
	return RTFM_ClosedNew(New, Delete, Size, NoThrow);
}

// operator new(size_t, std::nothrow_t const&)
// ??_U@YAPEAX_KAEBUnothrow_t@std@@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_ZnamRKSt9nothrow_t(size_t Size, std::nothrow_t const& NoThrow)
{
	void* (*New)(size_t, std::nothrow_t const& NoThrow) = &operator new[];
	void (*Delete)(void*, std::nothrow_t const& NoThrow) = &operator delete[];
	return RTFM_ClosedNew(New, Delete, Size, NoThrow);
}

// operator new(size_t, std::align_val_t)
// ??2@YAPEAX_KW4align_val_t@std@@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_ZnwmSt11align_val_t(size_t Size, std::align_val_t Align)
{
	void* (*New)(size_t, std::align_val_t) = &operator new;
	void (*Delete)(void*, std::align_val_t) = &operator delete;
	return RTFM_ClosedNew(New, Delete, Size, Align);
}

// operator new[](size_t, std::align_val_t)
// ??_U@YAPEAX_KW4align_val_t@std@@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_ZnamSt11align_val_t(size_t Size, std::align_val_t Align)
{
	void* (*New)(size_t, std::align_val_t) = &operator new[];
	void (*Delete)(void*, std::align_val_t) = &operator delete[];
	return RTFM_ClosedNew(New, Delete, Size, Align);
}

// operator new(size_t Size, std::align_val_t, std::nothrow_t const&)
// ??2@YAPEAX_KW4align_val_t@std@@AEBUnothrow_t@1@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_ZnwmSt11align_val_tRKSt9nothrow_t(
	size_t Size, std::align_val_t Align, std::nothrow_t const& NoThrow)
{
	void* (*New)(size_t, std::align_val_t, std::nothrow_t const&) = &operator new;
	void (*Delete)(void*, std::align_val_t, std::nothrow_t const&) = &operator delete;
	return RTFM_ClosedNew(New, Delete, Size, Align, NoThrow);
}

// operator new[](size_t Size, std::align_val_t, std::nothrow_t const&)
// ??_U@YAPEAX_KW4align_val_t@std@@AEBUnothrow_t@1@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void* RTFM_ZnamSt11align_val_tRKSt9nothrow_t(
	size_t Size, std::align_val_t Align, std::nothrow_t const& NoThrow)
{
	void* (*New)(size_t, std::align_val_t, std::nothrow_t const&) = &operator new[];
	void (*Delete)(void*, std::align_val_t, std::nothrow_t const&) = &operator delete[];
	return RTFM_ClosedNew(New, Delete, Size, Align, NoThrow);
}

// operator delete(void*)
// ??3@YAXPEAX@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdlPv(void* Pointer)
{
	void (*Delete)(void*) = &operator delete;
	RTFM_ClosedDelete(Delete, Pointer);
}

// operator delete[](void*)
// ??_V@YAXPEAX@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdaPv(void* Pointer)
{
	void (*Delete)(void*) = &operator delete[];
	RTFM_ClosedDelete(Delete, Pointer);
}

// operator delete(void*, std::nothrow_t const&)
// ??3@YAXPEAXAEBUnothrow_t@std@@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdlPvRKSt9nothrow_t(void* Pointer, std::nothrow_t const& NoThrow)
{
	void (*Delete)(void*, std::nothrow_t const&) = &operator delete;
	RTFM_ClosedDelete(Delete, Pointer, NoThrow);
}

// operator delete[](void *, std::nothrow_t const&)
// ??_V@YAXPEAXAEBUnothrow_t@std@@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdaPvRKSt9nothrow_t(void* Pointer, std::nothrow_t const& NoThrow)
{
	void (*Delete)(void*, std::nothrow_t const&) = &operator delete[];
	RTFM_ClosedDelete(Delete, Pointer, NoThrow);
}

// operator delete(void*, size_t)
// ??3@YAXPEAX_K@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdlPvm(void* Pointer, size_t Size)
{
	void (*Delete)(void*, size_t) = &operator delete;
	RTFM_ClosedDelete(Delete, Pointer, Size);
}

// operator delete[](void*, size_t)
// ??_V@YAXPEAX_K@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdaPvm(void* Pointer, size_t Size)
{
	void (*Delete)(void*, size_t) = &operator delete[];
	RTFM_ClosedDelete(Delete, Pointer, Size);
}

// operator delete(void*, std::align_val_t)
// ??3@YAXPEAXW4align_val_t@std@@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdlPvSt11align_val_t(void* Pointer, std::align_val_t Align)
{
	void (*Delete)(void*, std::align_val_t) = &operator delete;
	RTFM_ClosedDelete(Delete, Pointer, Align);
}

// operator delete[](void*, std::align_val_t)
// ??_V@YAXPEAXW4align_val_t@std@@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdaPvSt11align_val_t(void* Pointer, std::align_val_t Align)
{
	void (*Delete)(void*, std::align_val_t) = &operator delete[];
	RTFM_ClosedDelete(Delete, Pointer, Align);
}

// operator delete(void*, std::align_val_t, std::nothrow_t const&)
// ??3@YAXPEAXW4align_val_t@std@@AEBUnothrow_t@1@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdlPvSt11align_val_tRKSt9nothrow_t(
	void* Pointer, std::align_val_t Align, std::nothrow_t const& NoThrow)
{
	void (*Delete)(void*, std::align_val_t, std::nothrow_t const&) = &operator delete;
	RTFM_ClosedDelete(Delete, Pointer, Align, NoThrow);
}

// operator delete[](void*, std::align_val_t, std::nothrow_t const&)
// ??_V@YAXPEAXW4align_val_t@std@@AEBUnothrow_t@1@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdaPvSt11align_val_tRKSt9nothrow_t(
	void* Pointer, std::align_val_t Align, std::nothrow_t const& NoThrow)
{
	void (*Delete)(void*, std::align_val_t, std::nothrow_t const&) = &operator delete[];
	RTFM_ClosedDelete(Delete, Pointer, Align, NoThrow);
}

// operator delete(void*, size_t, std::align_val_t)
// ??3@YAXPEAX_KW4align_val_t@std@@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdlPvmSt11align_val_t(
	void* Pointer, unsigned long Size, std::align_val_t Align)
{
	void (*Delete)(void*, size_t, std::align_val_t) = &operator delete;
	RTFM_ClosedDelete(Delete, Pointer, Size, Align);
}

// operator delete[](void*, size_t, std::align_val_t)
// ??_V@YAXPEAX_KW4align_val_t@std@@@Z
extern "C" AUTORTFM_INTERNAL UE_AUTORTFM_API void RTFM_ZdaPvmSt11align_val_t(
	void* Pointer, unsigned long Size, std::align_val_t Align)
{
	void (*Delete)(void*, size_t, std::align_val_t) = &operator delete[];
	RTFM_ClosedDelete(Delete, Pointer, Size, Align);
}

}  // namespace AutoRTFM

#endif  // defined(__AUTORTFM) && __AUTORTFM
