// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/Constants.h"
#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Allocator.h"

#include <cstddef>

namespace AutoRTFM
{

// A stl-compatible allocator which wraps an AutoRTFM allocator.
// TODO(SOL-7652): Remove this once HashMap has been replaced with a bespoke
// hashmap implementation.
template <typename T, Allocator Allocator = FAllocator>
class [[clang::autortfm(autortfm_mode_internal)]] StlAllocator
{
public:
	using value_type = T;

	StlAllocator() = default;

	template <typename U, typename A>
	StlAllocator(const StlAllocator<U, A>&){};

	T* allocate(size_t Count)
	{
		return static_cast<T*>(Allocator::Allocate(Count * sizeof(T), alignof(T)));
	}

	void deallocate(T* Pointer, size_t Count)
	{
		Allocator::Free(Pointer, Count * sizeof(T));
	}
};

template <typename T, typename U, typename A>
bool operator==(const StlAllocator<T, A>&, const StlAllocator<U, A>&)
{
	return true;
}

template <typename T, typename U, typename A>
bool operator!=(const StlAllocator<T, A>&, const StlAllocator<U, A>&)
{
	return false;
}

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
