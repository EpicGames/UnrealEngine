// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "HashMap.h"
#include "Pool.h"
#include "Stack.h"
#include "Utils.h"

#include <cstddef>

namespace AutoRTFM
{

// A sequential linked-list of tasks, with the ability to tag some tasks with a
// key. Those tagged with a key can be removed from the list in LIFO order
// (stack-like).
// Tasks can be traversed bi-directionally.
//
// Template parameters:
// Task            - the task data type
// Key             - the data type used as a key for push / pop.
// InlineTaskCount - the number of tasks that can be allocated before a performing heap allocations.
template <typename TaskType, typename KeyType = const void*>
class AUTORTFM_DISABLE TTaskArray
{
	struct FEntry
	{
		// The task function.
		TaskType Task;
		// The next sequential task.
		FEntry* Next = nullptr;
		// The previous sequential task.
		FEntry* Prev = nullptr;

		// Constructor using copied Task
		FEntry(const TaskType& Task) : Task{Task} {}
		// Constructor using moved Task
		FEntry(TaskType&& Task) : Task{std::move(Task)} {}

		// Unlinks this entry from the doubly-linked list, updating the provided
		// head and tail pointers as necessary.
		void Unlink(FEntry*& ListHead, FEntry*& ListTail)
		{
			if (ListHead == this)
			{
				ListHead = Next;
			}
			if (ListTail == this)
			{
				ListTail = Prev;
			}
			if (Next)
			{
				Next->Prev = Prev;
			}
			if (Prev)
			{
				Prev->Next = Next;
			}
			Next = nullptr;
			Prev = nullptr;
		}

		// Links this entry to the doubly-linked list's tail pointer.
		// This entry must be unlinked before calling.
		void Link(FEntry*& ListHead, FEntry*& ListTail)
		{
			AUTORTFM_ASSERT(Next == nullptr && Prev == nullptr);
			if (!ListHead)
			{
				ListHead = this;
			}
			Prev = ListTail;
			if (Prev)
			{
				Prev->Next = this;
			}
			ListTail = this;
		}
	};

public:
	// The shared pool type for task arrays.
	using FEntryPool = TPool<FEntry, 256>;

	// Constructor.
	// TaskEntryPool is used to allocate new task entries and can be shared
	// between task arrays.
	TTaskArray(FEntryPool& TaskEntryPool) : EntryPool{TaskEntryPool} {}

	// Destructor. Returns all allocated tasks to the pool.
	~TTaskArray()
	{
		Reset();
	}

	// Clears the array and returns all allocated tasks to the pool.
	void Reset()
	{
		while (FEntry* Entry = Head)
		{
			Head = Head->Next;
			EntryPool.Return(Entry);
		}
		Tail = nullptr;
		Keyed.Empty();
		Count = 0;
	}

	// Adds the unkeyed task to the end of the array.
	template <typename TaskParamType>
	void Add(TaskParamType&& Task)
	{
		FEntry* Entry = EntryPool.Take(std::forward<TaskParamType>(Task));
		Entry->Link(Head, Tail);
		Count++;
	}

	// Adds the keyed task to the end of the array.
	template <typename TaskParamType>
	void AddKeyed(KeyType Key, TaskParamType&& Task)
	{
		Add(std::forward<TaskParamType>(Task));
		Keyed.FindOrAdd(Key).Push(Tail);
	}

	// Moves all the tasks from OtherArray to the end of this array, clearing
	// OtherArray.
	void AddAll(TTaskArray&& OtherArray)
	{
		if (Head)
		{
			// This array has tasks in the list.
			// Link the two task linked lists together.
			Tail->Next = OtherArray.Head;
			if (OtherArray.Head)
			{
				OtherArray.Head->Prev = Tail;
				Tail = OtherArray.Tail;
			}
		}
		else
		{
			// This array holds no tasks, so replace this list with OtherArray's.
			Head = OtherArray.Head;
			Tail = OtherArray.Tail;
		}

		// Append the keyed entries of OtherArray to this.
		for (auto& Iterator : OtherArray.Keyed)
		{
			KeyType Key = Iterator.Key;
			TEntryStack& EntryStack = Iterator.Value;
			Keyed.FindOrAdd(Key).PushAll(std::move(EntryStack));
		}

		// Add in OtherArray's count.
		Count += OtherArray.Count;

		// Everything stolen from OtherArray. Reset it.
		OtherArray.Head = nullptr;
		OtherArray.Tail = nullptr;
		OtherArray.Keyed.Reset();
		OtherArray.Count = 0;
	}

	// Removes the last added task with the given key.
	// Returns true if the task with the given key was removed, or false if
	// there are no remaining tasks with the given key.
	bool DeleteKey(KeyType Key)
	{
		if (TEntryStack* EntryStack = Keyed.Find(Key); EntryStack)
		{
			Release(EntryStack->Back());
			EntryStack->Pop();
			if (EntryStack->IsEmpty())
			{
				Keyed.Remove(Key);
			}
			return true;
		}

		return false;
	}

	// Removes all the tasks with the given key.
	// Returns true if a task with the given key was removed, or false if
	// there are no remaining tasks with the given key.
	bool DeleteAllMatchingKeys(KeyType Key)
	{
		if (TEntryStack* EntryStack = Keyed.Find(Key); EntryStack)
		{
			while (!EntryStack->IsEmpty())
			{
				Release(EntryStack->Back());
				EntryStack->Pop();
			}
			Keyed.Remove(Key);
			return true;
		}

		return false;
	}

	// Traverses the tasks from least recently added to most recently added,
	// calling Callback with each task, and removing each from the array.
	// Callback is a function-like type with the signature: void(TASK_TYPE&)
	template <typename CallbackType>
	void RemoveEachForward(CallbackType&& Callback)
	{
		while (FEntry* Entry = Head)
		{
			Callback(Entry->Task);
			Head = Entry->Next;
			EntryPool.Return(Entry);
		}
		Head = nullptr;
		Tail = nullptr;
		Keyed.Empty();
		Count = 0;
	}

	// Traverses the tasks from most recently added to least recently added,
	// calling Callback with each task, and removing each from the array.
	// Callback is a function-like type with the signature: void(TASK_TYPE&)
	template <typename CallbackType>
	void RemoveEachBackward(CallbackType&& Callback)
	{
		while (FEntry* Entry = Tail)
		{
			Callback(Entry->Task);
			Tail = Entry->Prev;
			EntryPool.Return(Entry);
		}
		Head = nullptr;
		Keyed.Empty();
		Count = 0;
	}

	// Returns the total number of tasks held by the array.
	size_t Num() const
	{
		return Count;
	}

	// Returns true if there are no tasks held by the array.
	bool IsEmpty() const
	{
		return Count == 0;
	}

private:
	// An stack of pointers to entries in the linked list.
	using TEntryStack = TStack<FEntry*, 8>;

	// Unlinks Entry and releases it back to the pool.
	// Also decrements the count.
	void Release(FEntry* Entry)
	{
		Entry->Unlink(Head, Tail);
		EntryPool.Return(Entry);
		Count--;
	}

	// The entry pool. Holds the underlying allocator.
	FEntryPool& EntryPool;

	// The doubly-linked list of tasks held by this array.
	FEntry* Head = nullptr;
	FEntry* Tail = nullptr;

	// A map of keyed tasks keys to their ordered task entries in the linked list.
	THashMap<KeyType, TEntryStack> Keyed;

	// Total number of tasks held by this array.
	size_t Count = 0;
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
