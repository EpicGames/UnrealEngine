// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM/Testing.h"

#if AUTORTFM_ENABLE_TEST_UTILS

#include "BuildMacros.h"
#include "ScopedGuard.h"
#include "Utils.h"

#include <unordered_map>
#include <vector>

#if AUTORTFM_PLATFORM_WINDOWS
#include "WindowsHeader.h"
#endif
#if AUTORTFM_PLATFORM_LINUX
#include <signal.h>
#include <stdio.h>
#include <string.h>
#endif

namespace AutoRTFM::Testing
{

#if UE_AUTORTFM
struct AUTORTFM_DISABLE FTrackingAllocator::FState
{
	FState(OnFailureFn OnFailure) : OnFailure{OnFailure}
	{
		if (Instance)
		{
			OnFailure("only one FTrackingAllocator instance can exist at any time");
		}
		Instance = this;
	}

	~FState()
	{
		Instance = nullptr;
	}

	void Install(autortfm_extern_api& API, bool bRecordAllocationStackTraces)
	{
		UnderlyingAllocator.Allocate = API.Allocate;
		UnderlyingAllocator.Reallocate = API.Reallocate;
		UnderlyingAllocator.AllocateZeroed = API.AllocateZeroed;
		UnderlyingAllocator.Free = API.Free;
		Log = API.Log;

		if (bRecordAllocationStackTraces)
		{
			LogCallstack = API.LogCallstack;
			CaptureCallstack = API.CaptureCallstack;
		}

		API.Allocate = [](size_t Size, size_t Alignment) -> void*
		{
			return Instance->Allocate(Size, Alignment, __builtin_return_address(0));
		};
		API.Reallocate = [](void* Pointer, size_t Size, size_t Alignment, size_t PreviousSize) -> void*
		{
			return Instance->Reallocate(Pointer, Size, Alignment, PreviousSize, __builtin_return_address(0));
		};
		API.AllocateZeroed = [](size_t Size, size_t Alignment) -> void*
		{
			return Instance->AllocateZeroed(Size, Alignment, __builtin_return_address(0));
		};
		API.Free = [](void* Pointer, size_t AllocationSize)
		{
			Instance->Free(Pointer, AllocationSize);
		};
	}

	void* Allocate(size_t Size, size_t Alignment, void* ReturnAddress)
	{
		void* NewAllocation = UnderlyingAllocator.Allocate(Size, Alignment);

		if (bRecursionGuard)
		{
			return NewAllocation;
		}
		AutoRTFM::TScopedGuard<bool> RecursionGuard(bRecursionGuard, true);

		if (auto It = Allocations.find(NewAllocation); It != Allocations.end())
		{
			OnFailure("Allocate() returned pointer already in use");
			MaybePrintCallstack(NewAllocation, It->second.Size, It->second.Callstack);
		}
		Allocations[NewAllocation] = FAllocationInfo{.Size = Size, .Callstack = MaybeCaptureCallstack(ReturnAddress)};
		return NewAllocation;
	}

	void* Reallocate(void* Pointer, size_t Size, size_t Alignment, size_t PreviousSize, void* ReturnAddress)
	{
		void* NewAllocation = UnderlyingAllocator.Reallocate(Pointer, Size, Alignment, PreviousSize);

		if (bRecursionGuard)
		{
			return NewAllocation;
		}
		AutoRTFM::TScopedGuard<bool> RecursionGuard(bRecursionGuard, true);

		if (Pointer)
		{
			if (auto It = Allocations.find(Pointer); It != Allocations.end())
			{
				if (It->second.Size != PreviousSize)
				{
					OnFailure("Reallocate() called with size that does not match the previously allocated size");
					MaybePrintCallstack(Pointer, It->second.Size, It->second.Callstack);
				}
				Allocations.erase(It);
			}
			else
			{
				OnFailure("Reallocate() called with pointer that was not previously allocated");
			}
		}

		if (auto It = Allocations.find(NewAllocation); It != Allocations.end())
		{
			OnFailure("Reallocate() returned pointer already in use");
			MaybePrintCallstack(NewAllocation, It->second.Size, It->second.Callstack);
		}

		Allocations[NewAllocation] = FAllocationInfo{.Size = Size, .Callstack = MaybeCaptureCallstack(ReturnAddress)};
		return NewAllocation;
	}

	void* AllocateZeroed(size_t Size, size_t Alignment, void* ReturnAddress)
	{
		void* NewAllocation = UnderlyingAllocator.AllocateZeroed(Size, Alignment);

		if (bRecursionGuard)
		{
			return NewAllocation;
		}
		AutoRTFM::TScopedGuard<bool> RecursionGuard(bRecursionGuard, true);

		if (Allocations.count(NewAllocation) != 0)
		{
			OnFailure("AllocateZeroed() returned pointer already in use");
		}
		Allocations[NewAllocation] = FAllocationInfo{.Size = Size, .Callstack = MaybeCaptureCallstack(ReturnAddress)};
		return NewAllocation;
	}

	void Free(void* Pointer, size_t AllocationSize)
	{
		UnderlyingAllocator.Free(Pointer, AllocationSize);

		if (bRecursionGuard)
		{
			return;
		}
		AutoRTFM::TScopedGuard<bool> RecursionGuard(bRecursionGuard, true);

		if (Pointer)
		{
			auto PreviousIt = Allocations.find(Pointer);
			if (PreviousIt == Allocations.end())
			{
				OnFailure("Free() called with pointer that was not previously allocated");
				MaybePrintCallstack(Pointer, PreviousIt->second.Size, PreviousIt->second.Callstack);
				return;
			}
			if (PreviousIt->second.Size != AllocationSize)
			{
				OnFailure("Free() called with size that does not match the previously allocated size");
				MaybePrintCallstack(Pointer, PreviousIt->second.Size, PreviousIt->second.Callstack);
			}
			Allocations.erase(PreviousIt);
		}
	}

	size_t TotalBytesAllocated() const
	{
		size_t Total = 0;
		for (auto It : Allocations)
		{
			Total += It.second.Size;
		}
		return Total;
	}

	void PrintAllocationCallstacks(size_t MaxCount) const
	{
		size_t Count = 0;
		for (auto It : Allocations)
		{
			if (Count >= MaxCount)
			{
				return;
			}
			MaybePrintCallstack(It.first, It.second.Size, It.second.Callstack);
			Count++;
		}
	}

private:
	std::vector<autortfm_callstack_frame> MaybeCaptureCallstack(void* ReturnAddress)
	{
		if (!CaptureCallstack)
		{
			return {};
		}

		static constexpr size_t MaxCallstack = 128;
		autortfm_callstack_frame Callstack[MaxCallstack];
		size_t const NumCaptured = CaptureCallstack(MaxCallstack, Callstack);

		// Attempt to trim off callstack frames under this function.
		size_t FramesToSkip = 0;
		for (size_t I = 0, N = std::min(NumCaptured, static_cast<size_t>(8)); I < N; I++)
		{
			if (reinterpret_cast<autortfm_callstack_frame>(ReturnAddress) == Callstack[I])
			{
				FramesToSkip += I;
				break;
			}
		}
		return std::vector<autortfm_callstack_frame>(&Callstack[FramesToSkip], &Callstack[NumCaptured - FramesToSkip]);
	}

	void MaybePrintCallstack(void* Address, size_t Size, std::vector<autortfm_callstack_frame>& Callstack) const
	{
		if (LogCallstack && Log && !Callstack.empty())
		{
			AUTORTFM_ERROR("AutoRTFM tracking allocator: Allocation %p of size %zu callstack:", Address, Size);
			LogCallstack(autortfm_log_error, Callstack.size(), Callstack.data());
		}
	}

	struct FUnderlyingAllocator
	{
		void* (*Allocate)(size_t Size, size_t Alignment);
		void* (*Reallocate)(void* Pointer, size_t Size, size_t Alignment, size_t PreviousSize);
		void* (*AllocateZeroed)(size_t Size, size_t Alignment);
		void (*Free)(void* Pointer, size_t AllocationSize);
	};

	struct FAllocationInfo
	{
		size_t Size = 0;
		std::vector<autortfm_callstack_frame> Callstack;
	};

	static FState* Instance;
	OnFailureFn OnFailure;
	FUnderlyingAllocator UnderlyingAllocator{};
	decltype(autortfm_extern_api::Log) Log = nullptr;
	decltype(autortfm_extern_api::LogCallstack) LogCallstack = nullptr;
	decltype(autortfm_extern_api::CaptureCallstack) CaptureCallstack = nullptr;
	std::unordered_map<void*, FAllocationInfo> Allocations{};
	bool bRecursionGuard = false;
};

FTrackingAllocator::FState* FTrackingAllocator::FState::Instance = nullptr;

FTrackingAllocator::FTrackingAllocator(OnFailureFn OnFailure) : State{new FState{OnFailure}} {}

FTrackingAllocator::~FTrackingAllocator()
{
	delete State;
	State = nullptr;
}

void FTrackingAllocator::Install(autortfm_extern_api& API, bool bRecordAllocationStackTraces /* = false */)
{
	State->Install(API, bRecordAllocationStackTraces);
}

size_t FTrackingAllocator::TotalBytesAllocated() const
{
	return State->TotalBytesAllocated();
}

void FTrackingAllocator::PrintAllocationCallstacks(size_t MaxCount /* = 5 */) const
{
	return State->PrintAllocationCallstacks(MaxCount);
}
#endif  // UE_AUTORTFM

static TTask<void(const FCrashInfo&)> GCrashHandlerCallback;

#if AUTORTFM_PLATFORM_WINDOWS
AUTORTFM_DISABLE static LONG WINAPI CrashHandler(_EXCEPTION_POINTERS* Info)
{
	const char* Kind = "unknown";
	switch (Info->ExceptionRecord->ExceptionCode)
	{
		case EXCEPTION_ACCESS_VIOLATION:
			Kind = "access violation";
			break;
		case EXCEPTION_ARRAY_BOUNDS_EXCEEDED:
			Kind = "array bounds exceeded";
			break;
		case EXCEPTION_DATATYPE_MISALIGNMENT:
			Kind = "datatype misalignment";
			break;
		case EXCEPTION_FLT_DENORMAL_OPERAND:
			Kind = "floating point denormal operand";
			break;
		case EXCEPTION_FLT_DIVIDE_BY_ZERO:
			Kind = "floating point divide by zero";
			break;
		case EXCEPTION_FLT_INEXACT_RESULT:
			Kind = "floating point inexact result";
			break;
		case EXCEPTION_FLT_INVALID_OPERATION:
			Kind = "floating point invalid operation";
			break;
		case EXCEPTION_FLT_OVERFLOW:
			Kind = "floating point overflow";
			break;
		case EXCEPTION_FLT_STACK_CHECK:
			Kind = "floating point stack check";
			break;
		case EXCEPTION_FLT_UNDERFLOW:
			Kind = "floating point underflow";
			break;
		case EXCEPTION_ILLEGAL_INSTRUCTION:
			Kind = "illegal instruction";
			break;
		case EXCEPTION_IN_PAGE_ERROR:
			Kind = "in page error";
			break;
		case EXCEPTION_INT_DIVIDE_BY_ZERO:
			Kind = "integer divide by zero";
			break;
		case EXCEPTION_INT_OVERFLOW:
			Kind = "integer overflow";
			break;
		case EXCEPTION_INVALID_DISPOSITION:
			Kind = "invalid disposition";
			break;
		case EXCEPTION_NONCONTINUABLE_EXCEPTION:
			Kind = "noncontinuable exception";
			break;
		case EXCEPTION_PRIV_INSTRUCTION:
			Kind = "private instruction";
			break;
		case EXCEPTION_STACK_OVERFLOW:
			Kind = "stack overflow";
			break;
		case 0x4000:  // see AssertExceptionCode in WindowsPlatformCrashContext.cpp
			Kind = "UE assertion failed";
			break;
		case 0x8000:  // see GPUCrashExceptionCode in WindowsPlatformCrashContext.cpp
			Kind = "UE GPU crash";
			break;
		case 0xc000:  // see OutOfMemoryExceptionCode in WindowsPlatformCrashContext.cpp
			Kind = "UE out of memory error";
			break;
		default:
			return EXCEPTION_CONTINUE_SEARCH;
	}
	void* ProgramCounter = nullptr;
#if AUTORTFM_ARCHITECTURE_X64
	ProgramCounter = reinterpret_cast<void*>(Info->ContextRecord->Rip);
#endif  // TODO: ARM64.
	GCrashHandlerCallback(FCrashInfo{.Kind = Kind, .ProgramCounter = ProgramCounter});
	return EXCEPTION_CONTINUE_SEARCH;
}
#endif

#if AUTORTFM_PLATFORM_LINUX
AUTORTFM_DISABLE static void CrashHandler(int Signal, siginfo_t* Info, void* Context)
{
	const char* Kind = strsignal(Signal);
	void* ProgramCounter = __builtin_return_address(0);
	switch (Signal)
	{
		case SIGILL:
		case SIGFPE:
		case SIGSEGV:
		case SIGBUS:
		{
			static char KindWithAddress[512];
			snprintf(KindWithAddress, sizeof(KindWithAddress), "%s (address: %p)", Kind, Info->si_addr);
			Kind = KindWithAddress;
			break;
		}
	}
	GCrashHandlerCallback(FCrashInfo{.Kind = Kind, .ProgramCounter = ProgramCounter});
	signal(SIGABRT, SIG_DFL);
	abort();
}
#endif

void OnCrash(TTask<void(const FCrashInfo&)> Callback)
{
	GCrashHandlerCallback = Callback;

#if AUTORTFM_PLATFORM_WINDOWS
	AddVectoredExceptionHandler(/* call last */ 0, &CrashHandler);
#endif

#if AUTORTFM_PLATFORM_LINUX
	struct sigaction Action = {};
	sigemptyset(&Action.sa_mask);
	Action.sa_flags = SA_SIGINFO;
	Action.sa_sigaction = &CrashHandler;
	sigaction(SIGILL, &Action, NULL);
	sigaction(SIGFPE, &Action, NULL);
	sigaction(SIGBUS, &Action, NULL);
	sigaction(SIGSEGV, &Action, NULL);
	sigaction(SIGSYS, &Action, NULL);
	sigaction(SIGABRT, &Action, NULL);
	sigaction(SIGTRAP, &Action, NULL);
#endif
}

}  // namespace AutoRTFM::Testing

#endif  // AUTORTFM_ENABLE_TEST_UTILS
