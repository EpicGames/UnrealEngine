// Copyright Epic Games, Inc. All Rights Reserved.

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "ThreadID.h"

#if AUTORTFM_PLATFORM_WINDOWS
#include "WindowsHeader.h"
#endif

namespace AutoRTFM
{

const FThreadID FThreadID::Invalid{};

FThreadID FThreadID::GetCurrent()
{
	// On Windows, call GetCurrentThreadId() directly.
	// This avoids an open write in std::this_thread::get_id(), which can cause
	// a stack overflow as autortfm_sanitizer_open_write() will call back into
	// this function.
#if AUTORTFM_PLATFORM_WINDOWS
	return FThreadID{GetCurrentThreadId()};
#else
	return FThreadID{std::this_thread::get_id()};
#endif
}

}

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
