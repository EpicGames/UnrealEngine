// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM.h"

#include "BuildMacros.h"

#if AUTORTFM_PLATFORM_WINDOWS
#include <cstdint>
#else
#include <thread>
#endif

namespace AutoRTFM
{

// A unique identifier for a thread of execution.
// Constructed as FThreadID::Invalid.
struct AUTORTFM_OPEN FThreadID
{
	// An invalid thread identifier.
	UE_AUTORTFM_API static const FThreadID Invalid;

	// Returns the currently executing thread's unique identifier.
	UE_AUTORTFM_API static FThreadID GetCurrent();

	// Equality operator
	inline bool operator==(const FThreadID& Other) const
	{
		return Value == Other.Value;
	}

	// Inequality operator
	inline bool operator!=(const FThreadID& Other) const
	{
		return Value != Other.Value;
	}

#if AUTORTFM_PLATFORM_WINDOWS
	uint32_t Value = 0;
#else
	std::thread::id Value;
#endif
};

}

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
