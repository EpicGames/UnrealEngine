// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM.h"

namespace AutoRTFM
{

inline const char* ToString(EContextStatus Status)
{
	switch (Status)
	{
		case EContextStatus::Idle:
			return "Idle";
		case EContextStatus::OnTrack:
			return "OnTrack";
		case EContextStatus::Unwinding:
			return "Unwinding";
		case EContextStatus::Committing:
			return "Committing";
		case EContextStatus::Aborting:
			return "Aborting";
		case EContextStatus::Retrying:
			return "Retrying";
		case EContextStatus::Completing:
			return "Completing";
		case EContextStatus::InStaticLocalInitializer:
			return "InStaticLocalInitializer";
	}
}

inline const char* ToString(ETransactionStatus Status)
{
	switch (Status)
	{
		case ETransactionStatus::Executing:
			return "Executing";
		case ETransactionStatus::Committed:
			return "Committed";
		case ETransactionStatus::AbortedByRequest:
			return "AbortedByRequest";
		case ETransactionStatus::AbortedByLanguage:
			return "AbortedByLanguage";
		case ETransactionStatus::AbortedByCascadingAbort:
			return "AbortedByCascadingAbort";
		case ETransactionStatus::AbortedByCascadingRetry:
			return "AbortedByCascadingRetry";
	}
}

inline const char* ToString(ETransactionResult Result)
{
	switch (Result)
	{
		case ETransactionResult::Committed:
			return "Committed";
		case ETransactionResult::AbortedByRequest:
			return "AbortedByRequest";
		case ETransactionResult::AbortedByLanguage:
			return "AbortedByLanguage";
		case ETransactionResult::RejectedTransactDuringUnwind:
			return "RejectedTransactDuringUnwind";
		case ETransactionResult::RejectedTransactDuringCommit:
			return "RejectedTransactDuringCommit";
		case ETransactionResult::RejectedTransactDuringAbort:
			return "RejectedTransactDuringAbort";
		case ETransactionResult::RejectedTransactDuringRetry:
			return "RejectedTransactDuringRetry";
		case ETransactionResult::RejectedTransactDuringCompletion:
			return "RejectedTransactDuringCompletion";
		case ETransactionResult::AbortedByCascadingAbort:
			return "AbortedByCascadingAbort";
	}
}

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
