// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - unused warnings

#include "BuildMacros.h"

namespace AutoRTFM
{

// Should we collect stats on the AutoRTFM runtime or not.
static constexpr bool bCollectStats = false;

// Tracking the locations of allocations is helpful for catching invalid frees.
#if AUTORTFM_BUILD_SHIPPING
static constexpr bool bTrackAllocationLocations = false;
#else
static constexpr bool bTrackAllocationLocations = true;
#endif

}  // namespace AutoRTFM
