// Copyright Epic Games, Inc. All Rights Reserved.

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Transaction.h"
#include "AutoRTFM.h"
#include "TransactionInlines.h"

#include "CallNestInlines.h"
#include "ContextInlines.h"
#include "HitSet.h"
#include "MetricsPriv.h"
#include "Sanitizer.h"
#include "Utils.h"

#include <cstdint>

namespace AutoRTFM
{

void FTransaction::Resurrect(FContext* const InContext)
{
	static TransactionID NextID = 1;

	AUTORTFM_ASSERT(!IsActive())
	AUTORTFM_ASSERT(IsFresh());
	Context = InContext;
	Parent = nullptr;
	bIsInAllocateFn = false;
	ID = NextID++;
}

void FTransaction::Suppress()
{
	AUTORTFM_ASSERT(IsActive())
	Context = nullptr;
	CommitTasks.Reset();
	PreAbortTasks.Reset();
	AbortTasks.Reset();

	HitSet.Reset();
	NewMemoryTracker.Reset();
	WriteLog.Reset();

	DeferredPopOnCommitHandlers.Reset();
	DeferredPopOnAbortHandlers.Reset();
	DeferredPopAllOnCommitHandlers.Reset();
	DeferredPopAllOnAbortHandlers.Reset();

	ID = 0;

	bIsInAllocateFn = false;

	AUTORTFM_ASSERT(IsFresh());
}

FTransaction** FTransaction::GetIntrusiveAddress()
{
	return &Parent;
}

FTransaction::FTransaction(FContext* InContext)
	: CommitTasks(InContext->GetTaskPool())
	, PreAbortTasks(InContext->GetTaskPool())
	, AbortTasks(InContext->GetTaskPool())
{
	Resurrect(InContext);
}

void FTransaction::Initialize(FTransaction* InParent, FStackRange InStackRange)
{
	Parent = InParent;
	StackRange = InStackRange;
	AUTORTFM_SANITIZER_ONLY(Sanitizer::StartTransaction(ID, StackRange));
}

bool FTransaction::IsFresh() const
{
	return HitSet.IsEmpty() && NewMemoryTracker.IsEmpty() && WriteLog.IsEmpty() && CommitTasks.IsEmpty() && PreAbortTasks.IsEmpty()
		&& AbortTasks.IsEmpty() && DeferredPopOnCommitHandlers.IsEmpty() && DeferredPopOnAbortHandlers.IsEmpty()
		&& DeferredPopAllOnCommitHandlers.IsEmpty() && DeferredPopAllOnAbortHandlers.IsEmpty();
}

void FTransaction::Abort()
{
	AUTORTFM_ASSERT(IsActive());
	AUTORTFM_ASSERT(Context->GetStatus() == EContextStatus::Aborting);

	UpdatePeakMemoryUsageMetrics();
	if (!Parent)
	{
		UpdateMemoryUsageHistogram();
	}

	// Call the destructors of all the OnCommit functors before undoing the transactional memory and
	// calling the OnAbort callbacks. This is important as the callback functions may have captured
	// variables that are depending on the allocated memory.
	CommitTasks.Reset();

	// Call each of the pre memory rollback on-abort handlers.
	PreAbortTasks.RemoveEachBackward([&](TTask<void()>& Task) { Task(); });

	// Revert all memory in the write-log.
	Undo();

	AUTORTFM_SANITIZER_ONLY(Sanitizer::AbortTransaction(ID));

	// Call each of the post memory rollback on-abort handlers.
	AbortTasks.RemoveEachBackward([&](TTask<void()>& Task) { Task(); });
}

void FTransaction::Commit()
{
	AUTORTFM_ASSERT(IsActive());
	AUTORTFM_ASSERT(Context->GetStatus() == EContextStatus::Committing);
	AUTORTFM_ASSERT(Context->GetCurrentTransaction() == this);
	AUTORTFM_SANITIZER_ONLY(Sanitizer::CommitTransaction(ID));

	if (Parent)
	{
		CommitNested();
	}
	else
	{
		CommitOuterNest();
	}
}

void FTransaction::Undo()
{
	AUTORTFM_ASSERT(IsActive());
	AUTORTFM_VERBOSE("Undoing a transaction...");

	// Prevent custom rollback functions from triggering the AutoRTFM sanitizer
	AUTORTFM_SANITIZER_DISABLE_SCOPE();

	for (auto Iter = WriteLog.rbegin(); Iter != WriteLog.rend(); ++Iter)
	{
		FWriteLogEntry Entry = *Iter;
		// No write records should be within the transaction's stack range.
		AUTORTFM_ENSURE(!IsOnStack(Entry.LogicalAddress));

		if ((Entry.Flags & EWriteFlags::CustomRollback) == EWriteFlags::CustomRollback)
		{
			GExternAPI.RollbackWrite(Entry.LogicalAddress, Entry.Data, Entry.Size, static_cast<autortfm_write_flags>(Entry.Flags));
		}
		else
		{
			memcpy(Entry.LogicalAddress, Entry.Data, Entry.Size);
		}
	}

	AUTORTFM_VERBOSE("Undone a transaction!");
}

void FTransaction::CommitNested()
{
	AUTORTFM_ASSERT(IsActive());
	AUTORTFM_ASSERT(Parent);

	// Disable the sanitizer for the scope of this function as we may allocate
	// memory, which may call autortfm_sanitizer_open_write().
	AUTORTFM_SANITIZER_DISABLE_SCOPE();

	// We need to pass our write log to our parent transaction, but with care!
	// We need to discard any writes if the memory location is on the parent
	// transaction's stack range.
	for (FWriteLogEntry Write : WriteLog)
	{
		if (Parent->IsOnStack(Write.LogicalAddress))
		{
			continue;
		}

		if (Write.Size <= FHitSet::MaxSize)
		{
			FHitSetEntry HitSetEntry{};
			HitSetEntry.Address = reinterpret_cast<uintptr_t>(Write.LogicalAddress);
			HitSetEntry.Size = static_cast<uint16_t>(Write.Size);
			HitSetEntry.Flags = static_cast<uintptr_t>(Write.Flags);

			if (Parent->HitSet.FindOrTryInsert(HitSetEntry))
			{
				continue;  // Don't duplicate the write-log entry.
			}
		}

		Parent->WriteLog.Push(Write);
	}

	// Similarly we need to merge the hit-sets to the parent, skipping those
	// that are on the parent transaction's stack.
	for (FHitSetEntry Entry : HitSet)
	{
		if (!Parent->IsOnStack(reinterpret_cast<void*>(Entry.Address)))
		{
			Parent->HitSet.FindOrTryInsert(Entry);
		}
	}

	// This point is likely the high-water mark for this transactional nest's memory usage.
	// It's after we've cloned our write log and hit set into the parent transaction,
	// but before we've cleared our deferred-task list.
	UpdatePeakMemoryUsageMetrics();

	// For all the deferred calls to `PopOnCommitHandler` that we couldn't
	// process (because our transaction nest didn't `PushOnCommitHandler`)
	// we need to move these to the parent now to handle them.
	for (const void* Key : DeferredPopOnCommitHandlers)
	{
		Parent->PopDeferUntilCommitHandler(Key);
	}
	DeferredPopOnCommitHandlers.Reset();

	// For all the deferred calls to `PopOnAbortHandler` that we couldn't
	// process (because our transaction nest didn't `PushOnAbortHandler`)
	// we need to move these to the parent now to handle them.
	for (const void* Key : DeferredPopOnAbortHandlers)
	{
		Parent->PopDeferUntilAbortHandler(Key);
	}
	DeferredPopOnAbortHandlers.Reset();

	// For all the calls to `PopAllOnCommitHandlers` we need to run these
	// again on parent now to handle them there too.
	for (const void* Key : DeferredPopAllOnCommitHandlers)
	{
		Parent->PopAllDeferUntilCommitHandlers(Key);
	}
	DeferredPopAllOnCommitHandlers.Reset();

	// For all the calls to `PopAllOnAbortHandlers` we need to run these
	// again on parent now to handle them there too.
	for (const void* Key : DeferredPopAllOnAbortHandlers)
	{
		Parent->PopAllDeferUntilAbortHandlers(Key);
	}
	DeferredPopAllOnAbortHandlers.Reset();

	Parent->CommitTasks.AddAll(std::move(CommitTasks));
	Parent->PreAbortTasks.AddAll(std::move(PreAbortTasks));
	Parent->AbortTasks.AddAll(std::move(AbortTasks));

	Parent->NewMemoryTracker.Merge(NewMemoryTracker);
}

void FTransaction::CommitOuterNest()
{
	AUTORTFM_ASSERT(IsActive());
	AUTORTFM_ASSERT(!Parent);

	UpdatePeakMemoryUsageMetrics();
	UpdateMemoryUsageHistogram();

	PreAbortTasks.Reset();
	AbortTasks.Reset();

	CommitTasks.RemoveEachForward([](TTask<void()>& Task) { Task(); });
}

}  // namespace AutoRTFM

#endif  // defined(__AUTORTFM) && __AUTORTFM
