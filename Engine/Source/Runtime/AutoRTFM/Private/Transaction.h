// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM.h"
#include "HitSet.h"
#include "IntrusivePool.h"
#include "MemoryTracker.h"
#include "Stack.h"
#include "StackRange.h"
#include "TaskArray.h"
#include "WriteLog.h"

namespace AutoRTFM
{
class FContext;

class AUTORTFM_INTERNAL FTransaction final
{
	friend class TIntrusivePool<FTransaction, 16>;

	void Resurrect(FContext* const InContext);
	void Suppress();
	FTransaction** GetIntrusiveAddress();

	// Constructor.
	FTransaction(FContext* const Context);

public:
	void Initialize(FTransaction* InParent, FStackRange InStackRange);

	FTransaction* GetParent() const
	{
		return Parent;
	}

	void DeferUntilCommit(TTask<void()>&&);
	void DeferUntilPreAbort(TTask<void()>&&);
	void DeferUntilAbort(TTask<void()>&&);
	void DeferUntilRetry(TTask<void()>&&);
	void PushDeferUntilCommitHandler(const void* Key, TTask<void()>&&);
	void PopDeferUntilCommitHandler(const void* Key);
	void PopAllDeferUntilCommitHandlers(const void* Key);
	void PushDeferUntilAbortHandler(const void* Key, TTask<void()>&&);
	void PopDeferUntilAbortHandler(const void* Key);
	void PopAllDeferUntilAbortHandlers(const void* Key);

	void Abort();
	void Commit();

	// Record that a write is about to occur at the given LogicalAddress of Size bytes.
	void RecordWrite(void* LogicalAddress, size_t Size, EWriteFlags Flags = EWriteFlags::Default);
	template <unsigned SIZE>
	void RecordWrite(void* LogicalAddress);
	template <unsigned SIZE>
	void RecordWriteInsertedSlow(void* LogicalAddress);
	template <unsigned SIZE>
	void RecordWriteNotInsertedSlow(void* LogicalAddress);

	void DidAllocate(void* LogicalAddress, size_t Size);
	void DidFree(void* LogicalAddress);

	// Returns true if the transaction is in its initial state.
	bool IsFresh() const;

	// Return true if this transaction is currently active.
	inline bool IsActive() const
	{
		return Context != nullptr;
	}

	inline TransactionID Identifier() const
	{
		return ID;
	}

	// The stack range represents all stack memory inside the transaction scope
	inline FStackRange GetStackRange() const
	{
		return StackRange;
	}

	// Returns true if the LogicalAddress is within the stack of the transaction.
	inline bool IsOnStack(const void* LogicalAddress) const;

private:
	using FTaskArray = TTaskArray<TTask<void()>>;

	void Undo();

	void CommitNested();
	void CommitOuterNest();

	bool ShouldRecordWrite(void* LogicalAddress) const;

	FContext* Context = nullptr;  // null indicates the transaction is not active

	// This field is used for two purposes:
	// - When the transaction is active, the Parent field is used to support transaction nesting.
	//   This field points to our immediate outer nest; it's null when this is the top level.
	// - When the transaction is "freed"--sitting in the FTransactionPool free list--this field is
	//   used to point to the next free item, forming a linked list of reusable FTransactions.
	//   (See GetIntrusiveAddress.)
	FTransaction* Parent = nullptr;

	// Commit tasks run on commit in forward order.
	FTaskArray CommitTasks;

	// Abort tasks run on abort (pre memory rollback) in reverse order.
	FTaskArray PreAbortTasks;

	// Abort tasks run on abort (post memory rollback) in reverse order.
	FTaskArray AbortTasks;

	// If a call to `PopOnCommitHandler` could not find a commit to pop, it is deferred
	// and tried again on the parent transaction.
	TStack<const void*, 8> DeferredPopOnCommitHandlers;

	// If a call to `PopOnAbortHandler` could not find an abort to pop, it is deferred
	// and tried again on the parent transaction.
	TStack<const void*, 8> DeferredPopOnAbortHandlers;

	// If a call to `PopAllOnCommitHandlers` was used and our transaction successfully
	// commits, we need to propagate this to the parent too.
	TStack<const void*, 1> DeferredPopAllOnCommitHandlers;

	// If a call to `PopAllOnAbortHandlers` was used and our transaction successfully
	// commits, we need to propagate this to the parent too.
	TStack<const void*, 1> DeferredPopAllOnAbortHandlers;

	FHitSet HitSet;
	FMemoryTracker NewMemoryTracker;
	FWriteLog WriteLog;
	FStackRange StackRange;
	TransactionID ID = 0;
	bool bIsInAllocateFn = false;
};

}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
