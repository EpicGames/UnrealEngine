// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM/Defines.h"
#include "AutoRTFM/ReportHazard.h"
#include "HitSet.h"
#include "Sanitizer.h"
#include "ScopedGuard.h"
#include "Toggles.h"
#include "Transaction.h"
#include "Utils.h"
#include "WriteLog.h"

#include <utility>

#if __has_include(<sanitizer/asan_interface.h>)
#include <sanitizer/asan_interface.h>
#if defined(__SANITIZE_ADDRESS__)
#define AUTORTFM_CHECK_ASAN_FAKE_STACKS 1
#elif defined(__has_feature)
#if __has_feature(address_sanitizer)
#define AUTORTFM_CHECK_ASAN_FAKE_STACKS 1
#endif
#endif
#endif

#ifndef AUTORTFM_CHECK_ASAN_FAKE_STACKS
#define AUTORTFM_CHECK_ASAN_FAKE_STACKS 0
#endif

namespace AutoRTFM
{

UE_AUTORTFM_FORCEINLINE bool FTransaction::IsOnStack(const void* LogicalAddress) const
{
	if (StackRange.Contains(LogicalAddress))
	{
		return true;
	}

#if AUTORTFM_CHECK_ASAN_FAKE_STACKS
	if (void* FakeStack = __asan_get_current_fake_stack())
	{
		void* Beg = nullptr;
		void* End = nullptr;
		void* RealAddress = __asan_addr_is_in_fake_stack(FakeStack, const_cast<void*>(LogicalAddress), &Beg, &End);
		return RealAddress && StackRange.Contains(RealAddress);
	}
#endif  // AUTORTFM_CHECK_ASAN_FAKE_STACKS

	return false;
}

UE_AUTORTFM_FORCEINLINE bool FTransaction::ShouldRecordWrite(void* LogicalAddress) const
{
	// We cannot record writes to stack memory used within the transaction, as
	// undoing the writes may corrupt stack memory that has been unwound or
	// is now being used for a different variable from the one the write was
	// made.
	return !IsOnStack(LogicalAddress);
}

AUTORTFM_NO_ASAN UE_AUTORTFM_FORCEINLINE void FTransaction::RecordWrite(
	void* LogicalAddress, size_t Size, EWriteFlags Flags /* = EWriteFlags::Default */)
{
	if (AUTORTFM_UNLIKELY(0 == Size))
	{
		return;
	}

	if (!ShouldRecordWrite(LogicalAddress))
	{
		return;
	}

	if (Size <= FHitSet::MaxSize)
	{
		FHitSetEntry HitSetEntry{};
		HitSetEntry.Address = reinterpret_cast<uintptr_t>(LogicalAddress);
		HitSetEntry.Size = static_cast<uint16_t>(Size);
		HitSetEntry.Flags = static_cast<uintptr_t>(Flags);

		if (HitSet.FindOrTryInsert(HitSetEntry))
		{
			return;
		}
	}

	if (NewMemoryTracker.Contains(LogicalAddress, Size))
	{
		return;
	}

	FWriteLogEntry Entry{.LogicalAddress = static_cast<std::byte*>(LogicalAddress),
		.Data = static_cast<std::byte*>(LogicalAddress),
		.Size = Size,
		.Flags = Flags};

	AutoRTFM::TScopedGuard<bool> RecursionGuard(bIsInAllocateFn, true);

	switch (Entry.Size)
	{
		case 8:
			WriteLog.PushSmall<8>(Entry.LogicalAddress, Entry.Data, Entry.Flags);
			break;
		case 4:
			WriteLog.PushSmall<4>(Entry.LogicalAddress, Entry.Data, Entry.Flags);
			break;
		case 2:
			WriteLog.PushSmall<2>(Entry.LogicalAddress, Entry.Data, Entry.Flags);
			break;
		default:
			WriteLog.Push(Entry);
			break;
	}

#if AUTORTFM_SANITIZER
	if ((Flags & EWriteFlags::NoSanitize) != EWriteFlags::NoSanitize)
	{
		Sanitizer::ClosedWrite(ID, LogicalAddress, Size, __builtin_return_address(0));
	}
#endif
}

template <unsigned SIZE>
AUTORTFM_NO_ASAN UE_AUTORTFM_FORCEINLINE void FTransaction::RecordWrite(void* LogicalAddress)
{
	static_assert(SIZE <= 8);
	if (!ShouldRecordWrite(LogicalAddress))
	{
		return;
	}

	FHitSetEntry Entry{};
	Entry.Address = reinterpret_cast<uintptr_t>(LogicalAddress);
	Entry.Size = static_cast<uint16_t>(SIZE);

	switch (HitSet.FindOrTryInsertNoResize(Entry))
	{
		case FHitSet::EInsertResult::Exists:
			return;
		case FHitSet::EInsertResult::Inserted:
			AUTORTFM_MUST_TAIL return FTransaction::RecordWriteInsertedSlow<SIZE>(LogicalAddress);
		case FHitSet::EInsertResult::NotInserted:
			AUTORTFM_MUST_TAIL return FTransaction::RecordWriteNotInsertedSlow<SIZE>(LogicalAddress);
	}
}

template <unsigned SIZE>
AUTORTFM_NO_ASAN UE_AUTORTFM_FORCENOINLINE void FTransaction::RecordWriteNotInsertedSlow(void* LogicalAddress)
{
	FHitSetEntry Entry{};
	Entry.Address = reinterpret_cast<uintptr_t>(LogicalAddress);
	Entry.Size = static_cast<uint16_t>(SIZE);

	if (HitSet.FindOrTryInsert(Entry))
	{
		return;
	}

	return RecordWriteInsertedSlow<SIZE>(LogicalAddress);
}

template <unsigned SIZE>
AUTORTFM_NO_ASAN UE_AUTORTFM_FORCENOINLINE void FTransaction::RecordWriteInsertedSlow(void* LogicalAddress)
{
	if (NewMemoryTracker.Contains(LogicalAddress, SIZE))
	{
		return;
	}

	AutoRTFM::TScopedGuard<bool> RecursionGuard(bIsInAllocateFn, true);

	WriteLog.PushSmall<SIZE>(static_cast<std::byte*>(LogicalAddress), static_cast<std::byte*>(LogicalAddress));

	AUTORTFM_SANITIZER_ONLY(Sanitizer::ClosedWrite(ID, LogicalAddress, SIZE, __builtin_return_address(0)));
}

UE_AUTORTFM_FORCEINLINE void FTransaction::DidAllocate(void* LogicalAddress, const size_t Size)
{
	if (0 == Size || bIsInAllocateFn)
	{
		return;
	}

	AutoRTFM::TScopedGuard<bool> RecursionGuard(bIsInAllocateFn, true);
	const bool DidInsert = NewMemoryTracker.Insert(LogicalAddress, Size);
	AUTORTFM_ASSERT(DidInsert);
}

UE_AUTORTFM_FORCEINLINE void FTransaction::DidFree(void* LogicalAddress)
{
	AUTORTFM_ASSERT(bTrackAllocationLocations);

	// Checking if one byte is in the interval map is enough to ascertain if it
	// is new memory and we should be worried.
	if (!bIsInAllocateFn)
	{
		if (NewMemoryTracker.Contains(LogicalAddress, 1))
		{
			AutoRTFM::TScopedGuard<bool> RecursionGuard(bIsInAllocateFn, true);
			ForTheRuntime::ReportAutoRTFMHazard(ForTheRuntime::EHazardType::OpenFree);
			AUTORTFM_REPORT_ERROR("Transactional memory was freed from the open.");
		}
	}
}

UE_AUTORTFM_FORCEINLINE void FTransaction::DeferUntilCommit(TTask<void()>&& Callback)
{
	// We explicitly must copy the function here because the original was allocated
	// within a transactional context, and thus the memory is allocating under
	// transactionalized conditions. By copying, we create an open copy of the callback.
	TTask<void()> Copy(Callback);
	CommitTasks.Add(std::move(Copy));
}

UE_AUTORTFM_FORCEINLINE void FTransaction::DeferUntilPreAbort(TTask<void()>&& Callback)
{
	// We explicitly must copy the function here because the original was allocated
	// within a transactional context, and thus the memory is allocating under
	// transactionalized conditions. By copying, we create an open copy of the callback.
	TTask<void()> Copy(Callback);
	PreAbortTasks.Add(std::move(Copy));
}

UE_AUTORTFM_FORCEINLINE void FTransaction::DeferUntilAbort(TTask<void()>&& Callback)
{
	// We explicitly must copy the function here because the original was allocated
	// within a transactional context, and thus the memory is allocating under
	// transactionalized conditions. By copying, we create an open copy of the callback.
	TTask<void()> Copy(Callback);
	AbortTasks.Add(std::move(Copy));
}

UE_AUTORTFM_FORCEINLINE void FTransaction::PushDeferUntilCommitHandler(const void* Key, TTask<void()>&& Callback)
{
	// We explicitly must copy the function here because the original was allocated
	// within a transactional context, and thus the memory was allocated under
	// transactionalized conditions. By copying, we create an open copy of the callback.
	TTask<void()> Copy(Callback);
	CommitTasks.AddKeyed(Key, std::move(Copy));
}

UE_AUTORTFM_FORCEINLINE void FTransaction::PopDeferUntilCommitHandler(const void* Key)
{
	if (AUTORTFM_LIKELY(CommitTasks.DeleteKey(Key)))
	{
		return;
	}

	DeferredPopOnCommitHandlers.Push(Key);
}

UE_AUTORTFM_FORCEINLINE void FTransaction::PopAllDeferUntilCommitHandlers(const void* Key)
{
	CommitTasks.DeleteAllMatchingKeys(Key);

	// We also need to remember to run this on our parent's nest if our transaction commits.
	DeferredPopAllOnCommitHandlers.Push(Key);
}

UE_AUTORTFM_FORCEINLINE void FTransaction::PushDeferUntilAbortHandler(const void* Key, TTask<void()>&& Callback)
{
	// We explicitly must copy the function here because the original was allocated
	// within a transactional context, and thus the memory is allocating under
	// transactionalized conditions. By copying, we create an open copy of the callback.
	TTask<void()> Copy(Callback);
	AbortTasks.AddKeyed(Key, std::move(Copy));
}

UE_AUTORTFM_FORCEINLINE void FTransaction::PopDeferUntilAbortHandler(const void* Key)
{
	if (AUTORTFM_LIKELY(AbortTasks.DeleteKey(Key)))
	{
		return;
	}

	DeferredPopOnAbortHandlers.Push(Key);
}

UE_AUTORTFM_FORCEINLINE void FTransaction::PopAllDeferUntilAbortHandlers(const void* Key)
{
	AbortTasks.DeleteAllMatchingKeys(Key);

	// We also need to remember to run this on our parent's nest if our transaction commits.
	DeferredPopAllOnAbortHandlers.Push(Key);
}

}  // namespace AutoRTFM

#undef AUTORTFM_CHECK_ASAN_FAKE_STACKS

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
