// Copyright Epic Games, Inc. All Rights Reserved.

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Utils.h"

#include "BuildMacros.h"
#include "ContextInlines.h"

#include <utility>

#if AUTORTFM_PLATFORM_WINDOWS
#include "WindowsHeader.h"
#include <dbghelp.h>
#else
#include <execinfo.h>
#endif

namespace AutoRTFM
{

void DoAssert(void (*Logger)())
{
	Logger();
	__builtin_unreachable();
}

void DoExpect(void (*Logger)())
{
	Logger();
}

std::string GetFunctionDescription(void* FunctionPtr)
{
#if AUTORTFM_PLATFORM_WINDOWS
	// This is gross, but it works. It's possible for someone to have SymInitialized before. But if they had, then this
	// will just fail. Also, this function is called in cases where we're failing, so it's ok if we do dirty things.
	SymInitialize(GetCurrentProcess(), nullptr, true);

	DWORD64 Displacement = 0;
	DWORD64 Address = reinterpret_cast<DWORD64>(FunctionPtr);
	char Buffer[sizeof(SYMBOL_INFO) + MAX_SYM_NAME * sizeof(TCHAR)];
	PSYMBOL_INFO Symbol = reinterpret_cast<PSYMBOL_INFO>(Buffer);
	Symbol->SizeOfStruct = sizeof(SYMBOL_INFO);
	Symbol->MaxNameLen = MAX_SYM_NAME;
	if (SymFromAddr(GetCurrentProcess(), Address, &Displacement, Symbol))
	{
		return Symbol->Name;
	}
	else
	{
		return "<error getting description>";
	}
#else   // AUTORTFM_PLATFORM_WINDOWS -> so !AUTORTFM_PLATFORM_WINDOWS
	char** const symbols = backtrace_symbols(&FunctionPtr, 1);
	std::string Name(*symbols);
	free(symbols);
	return Name;
#endif  // AUTORTFM_PLATFORM_WINDOWS -> so !AUTORTFM_PLATFORM_WINDOWS
}

UE_AUTORTFM_API UE_AUTORTFM_FORCENOINLINE void ReportError(const char* File, int Line, void* ProgramCounter, const char* Format, ...)
{
	if (ProgramCounter == nullptr)
	{
		ProgramCounter = __builtin_return_address(0);
	}

	const ForTheRuntime::EAutoRTFMInternalAbortActionState InternalAbortAction = ForTheRuntime::GetInternalAbortAction();

	switch (InternalAbortAction)
	{
		case ForTheRuntime::EAutoRTFMInternalAbortActionState::Crash:
			if (Format)
			{
				va_list Args;
				va_start(Args, Format);
				::AutoRTFM::LogV(File, Line, ProgramCounter, autortfm_log_fatal, Format, Args);
				va_end(Args);
			}
			else
			{
				::AutoRTFM::Log(File, Line, ProgramCounter, autortfm_log_fatal, "Transaction failing because of internal issue");
			}
			// The `autortfm_log_fatal` lines above probably triggered a crash already, but if we made it this far, force the crash.
			__builtin_trap();

		case ForTheRuntime::EAutoRTFMInternalAbortActionState::Abort:
			if (ForTheRuntime::GetEnsureOnInternalAbort())
			{
				if (Format == nullptr)
				{
					Format = "Transaction failing because of internal issue";
				}
				va_list Args;
				va_start(Args, Format);
				[[maybe_unused]] static bool bCalled = [&]
				{
					::AutoRTFM::EnsureFailureV(File, Line, ProgramCounter, "!GetEnsureOnInternalAbort()", Format, Args);
					return true;
				}();
				va_end(Args);
			}
			if (FContext* const Context = FContext::Get())
			{
				Context->AbortTransactionAndThrow(ETransactionStatus::AbortedByLanguage);
			}
			break;

		default:
			::AutoRTFM::EnsureFailure(
				File, Line, ProgramCounter, "Unexpected InternalAbortAction", "InternalAbortAction = %d", int(InternalAbortAction));
			break;
	}
}

}  // namespace AutoRTFM

#endif  // defined(__AUTORTFM) && __AUTORTFM
