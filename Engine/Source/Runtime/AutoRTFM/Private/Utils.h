// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "AutoRTFM/Defines.h"
#include "BuildMacros.h"
#include "ExternAPI.h"

#include <cstdarg>
#include <string>
#include <type_traits>

// If 0 then verbose logging is compiled-out.
#define AUTORTFM_VERBOSE_ENABLED 0

// AutoRTFM internal-runtime-only attribute that can be applied to classes,
// methods or functions to prevent AutoRTFM closed function(s) from being
// generated. Applying the annotation to a class is equivalent to applying the
// annotation to each method of the class.
// When annotating a function, the attribute must be placed on a the function
// declaration (usually in the header) and not a function implementation.
// Annotated functions are callable from both the open and closed, without
// trampolines, and the function is always uninstrumented.
// Note: This annotation will prevent inlining of annotated functions.
#define AUTORTFM_INTERNAL [[clang::autortfm(autortfm_mode_internal)]]

namespace AutoRTFM
{

// Reports an internal AutoRTFM issue. The behavior of this function will vary
// based on ForTheRuntime::GetInternalAbortAction() and
// ForTheRuntime::GetEnsureOnInternalAbort().
// If ProgramCounter is null, then the return address of the call will be used.
AUTORTFM_DISABLE __attribute__((__format__(__printf__, 4, 5))) UE_AUTORTFM_API void ReportError(
	const char* File, int Line, void* ProgramCounter, const char* Format, ...);

[[noreturn]] inline void InternalUnreachable();

std::string GetFunctionDescription(void* FunctionPtr);

template <typename TReturnType, typename... TParameterTypes>
std::string GetFunctionDescription(TReturnType (*FunctionPtr)(TParameterTypes...))
{
	return GetFunctionDescription(reinterpret_cast<void*>(FunctionPtr));
}

// We use a special calling convention for this call to minimize
// the cost of calling this function (which is super unlikely to
// be called because it is the asserting true codepath, EG. only
// fatal process destroying explosions happen here).
[[clang::preserve_most, noreturn]] UE_AUTORTFM_API UE_AUTORTFM_FORCENOINLINE void DoAssert(void (*Logger)());

// We use a special calling convention for this call to minimize
// the cost of calling this function (which is super unlikely to
// be called because it is the expecting true codepath, EG. only
// things we don't expect to happen get here).
[[clang::preserve_most]] UE_AUTORTFM_FORCENOINLINE void DoExpect(void (*Logger)());

// Logs a message using a printf-style format string and va_list arguments.
inline void LogV(const char* File, int Line, void* ProgramCounter, autortfm_log_severity Severity, const char* Format, va_list Args)
{
	if (ProgramCounter == nullptr)
	{
		ProgramCounter = __builtin_return_address(0);
	}

	GExternAPI.Log(File, Line, ProgramCounter, Severity, Format, Args);
}

// Logs a message using a printf-style format string and variadic arguments.
__attribute__((__format__(__printf__, 5, 6))) inline void Log(
	const char* File, int Line, void* ProgramCounter, autortfm_log_severity Severity, const char* Format, ...)
{
	if (ProgramCounter == nullptr)
	{
		ProgramCounter = __builtin_return_address(0);
	}

	va_list Args;
	va_start(Args, Format);
	GExternAPI.Log(File, Line, ProgramCounter, Severity, Format, Args);
	va_end(Args);
}

// Logs a message with a callstack using a printf-style format string and va_list arguments.
inline void LogWithCallstackV(autortfm_log_severity Severity, const char* Format, va_list Args)
{
	GExternAPI.LogWithCallstack(__builtin_return_address(0), Severity, Format, Args);
}

// Logs a message with a callstack using a printf-style format string and variadic arguments.
__attribute__((__format__(__printf__, 2, 3))) inline void LogWithCallstack(autortfm_log_severity Severity, const char* Format, ...)
{
	va_list Args;
	va_start(Args, Format);
	GExternAPI.LogWithCallstack(__builtin_return_address(0), Severity, Format, Args);
	va_end(Args);
}

// Reports an ensure failure using a printf-style format string and va_list arguments.
// If ProgramCounter is null, then the return address of the call will be used.
inline void EnsureFailureV(const char* File, int Line, void* ProgramCounter, const char* Condition, const char* Format, va_list Args)
{
	if (ProgramCounter == nullptr)
	{
		ProgramCounter = __builtin_return_address(0);
	}

	GExternAPI.EnsureFailure(File, Line, ProgramCounter, Condition, Format, Args);
}

// Reports an ensure failure using a printf-style format string and variadic arguments.
// If ProgramCounter is null, then the return address of the call will be used.
__attribute__((__format__(__printf__, 5, 6))) inline void EnsureFailure(
	const char* File, int Line, void* ProgramCounter, const char* Condition, const char* Format, ...)
{
	if (ProgramCounter == nullptr)
	{
		ProgramCounter = __builtin_return_address(0);
	}

	va_list Args;
	va_start(Args, Format);
	GExternAPI.EnsureFailure(File, Line, ProgramCounter, Condition, Format, Args);
	va_end(Args);
}

// Rounds Value down to the next multiple of Alignment.
// Alignment must be a power of two.
template <typename T>
inline constexpr T AlignDown(T Value, T Alignment)
{
	static_assert(std::is_integral_v<T>);
	return Value & ~(Alignment - 1);
}

// Rounds Value up to the next multiple of Alignment.
// Alignment must be a power of two.
template <typename T>
inline constexpr T AlignUp(T Value, T Alignment)
{
	static_assert(std::is_integral_v<T>);
	return (Value + Alignment - 1) & ~(Alignment - 1);
}

// Rounds Value down to the next multiple of Multiple.
// Unlike AlignDown(), Multiple does not need to be a power of two.
template <typename T>
inline constexpr T RoundDown(T Value, T Multiple)
{
	static_assert(std::is_integral_v<T>);
	return (Value / Multiple) * Multiple;
}

// Rounds Value up to the next multiple of Multiple.
// Unlike AlignUp(), Multiple does not need to be a power of two.
template <typename T>
inline constexpr T RoundUp(T Value, T Multiple)
{
	static_assert(std::is_integral_v<T>);
	return RoundDown(Value + Multiple - 1, Multiple);
}

// Returns the log2 of Value
template <typename T>
requires(std::is_integral_v<T> && std::is_unsigned_v<T>)
inline constexpr T Log2(T Value)
{
	constexpr size_t NumBits = sizeof(Value) * 8;
	return Value > 0 ? NumBits - __builtin_clzg(Value) - 1 : 0;
}

// RAII helper to assign a value to a variable for the lifetime of the
// FScopedAssignment, restoring the original value when the FScopedAssignment is
// destructed.
template <typename T>
struct TScopedAssignment
{
	inline TScopedAssignment(T& Variable, T NewValue) : Variable(Variable), OldValue(Variable)
	{
		Variable = NewValue;
	}

	inline ~TScopedAssignment()
	{
		Variable = OldValue;
	}

private:
	TScopedAssignment(const TScopedAssignment&) = delete;
	TScopedAssignment(TScopedAssignment&&) = delete;
	TScopedAssignment& operator=(const TScopedAssignment&) = delete;
	TScopedAssignment& operator=(TScopedAssignment&&) = delete;
	T& Variable;
	T OldValue;
};

// Assigns Value to Variable, and restores the original value of Variable at the
// end of the current scope.
#define AUTORTFM_SCOPED_ASSIGNMENT(Variable, Value) \
	TScopedAssignment UE_AUTORTFM_CONCAT(ScopedAssignment, __COUNTER__){Variable, Value};

}  // namespace AutoRTFM

#define AUTORTFM_REQUIRE_SEMICOLON static_assert(true)

#define AUTORTFM_REPORT_ERROR(Format, ...) \
	::AutoRTFM::ReportError(__FILE__, __LINE__, /* ProgramCounter */ nullptr, Format, ##__VA_ARGS__)

#define AUTORTFM_LOG_AT_SEVERITY(Severity, Format, ...) \
	::AutoRTFM::Log(__FILE__, __LINE__, /* ProgramCounter */ nullptr, Severity, Format, ##__VA_ARGS__)

#define AUTORTFM_VERBOSE(Format, ...) AUTORTFM_LOG_AT_SEVERITY(autortfm_log_verbose, Format, ##__VA_ARGS__)
#define AUTORTFM_LOG(Format, ...) AUTORTFM_LOG_AT_SEVERITY(autortfm_log_info, Format, ##__VA_ARGS__)
#define AUTORTFM_WARN(Format, ...) AUTORTFM_LOG_AT_SEVERITY(autortfm_log_warn, Format, ##__VA_ARGS__)
#define AUTORTFM_ERROR(Format, ...) AUTORTFM_LOG_AT_SEVERITY(autortfm_log_error, Format, ##__VA_ARGS__)
#define AUTORTFM_FATAL(Format, ...) AUTORTFM_LOG_AT_SEVERITY(autortfm_log_fatal, Format, ##__VA_ARGS__)

#define AUTORTFM_VERBOSE_V(Format, Args) \
	::AutoRTFM::LogV(__FILE__, __LINE__, /* ProgramCounter */ nullptr, autortfm_log_verbose, Format, Args)
#define AUTORTFM_LOG_V(Format, Args) \
	::AutoRTFM::LogV(__FILE__, __LINE__, /* ProgramCounter */ nullptr, autortfm_log_info, Format, Args)
#define AUTORTFM_WARN_V(Format, Args) \
	::AutoRTFM::LogV(__FILE__, __LINE__, /* ProgramCounter */ nullptr, autortfm_log_warn, Format, Args)
#define AUTORTFM_ERROR_V(Format, Args) \
	::AutoRTFM::LogV(__FILE__, __LINE__, /* ProgramCounter */ nullptr, autortfm_log_error, Format, Args)
#define AUTORTFM_FATAL_V(Format, Args) \
	::AutoRTFM::LogV(__FILE__, __LINE__, /* ProgramCounter */ nullptr, autortfm_log_fatal, Format, Args)

#define AUTORTFM_VERBOSE_IF(Condition, Format, ...)  \
	do                                               \
	{                                                \
		if (Condition)                               \
		{                                            \
			AUTORTFM_VERBOSE(Format, ##__VA_ARGS__); \
		}                                            \
	} while (false)
#define AUTORTFM_LOG_IF(Condition, Format, ...)  \
	do                                           \
	{                                            \
		if (Condition)                           \
		{                                        \
			AUTORTFM_LOG(Format, ##__VA_ARGS__); \
		}                                        \
	} while (false)
#define AUTORTFM_WARN_IF(Condition, Format, ...)  \
	do                                            \
	{                                             \
		if (Condition)                            \
		{                                         \
			AUTORTFM_WARN(Format, ##__VA_ARGS__); \
		}                                         \
	} while (false)
#define AUTORTFM_ERROR_IF(Condition, Format, ...)  \
	do                                             \
	{                                              \
		if (AUTORTFM_UNLIKELY(Condition))          \
		{                                          \
			AUTORTFM_ERROR(Format, ##__VA_ARGS__); \
		}                                          \
	} while (false)
#define AUTORTFM_FATAL_IF(Condition, Format, ...)  \
	do                                             \
	{                                              \
		if (AUTORTFM_UNLIKELY(Condition))          \
		{                                          \
			AUTORTFM_FATAL(Format, ##__VA_ARGS__); \
		}                                          \
	} while (false)

// If AUTORTFM_VERBOSE_ENABLED is 0, then replace all the verbose logging macros
// with no-ops.
#if !AUTORTFM_VERBOSE_ENABLED
#undef AUTORTFM_VERBOSE
#undef AUTORTFM_VERBOSE_V
#undef AUTORTFM_VERBOSE_IF
#define AUTORTFM_VERBOSE(...) AUTORTFM_REQUIRE_SEMICOLON
#define AUTORTFM_VERBOSE_V(...) AUTORTFM_REQUIRE_SEMICOLON
#define AUTORTFM_VERBOSE_IF(...) AUTORTFM_REQUIRE_SEMICOLON
#endif

#define AUTORTFM_ENSURE(Condition)                                                                                \
	do                                                                                                            \
	{                                                                                                             \
		if (AUTORTFM_UNLIKELY(!(Condition)))                                                                      \
		{                                                                                                         \
			[[maybe_unused]] static bool bCalled = []                                                             \
			{                                                                                                     \
				::AutoRTFM::EnsureFailure(__FILE__, __LINE__, /* ProgramCounter */ nullptr, #Condition, nullptr); \
				return true;                                                                                      \
			}();                                                                                                  \
		}                                                                                                         \
	} while (false)

#define AUTORTFM_ENSURE_MSG(Condition, Format, ...)                                                                             \
	do                                                                                                                          \
	{                                                                                                                           \
		if (AUTORTFM_UNLIKELY(!(Condition)))                                                                                    \
		{                                                                                                                       \
			[[maybe_unused]] static bool bCalled = [&]                                                                          \
			{                                                                                                                   \
				::AutoRTFM::EnsureFailure(__FILE__, __LINE__, /* ProgramCounter */ nullptr, #Condition, Format, ##__VA_ARGS__); \
				return true;                                                                                                    \
			}();                                                                                                                \
		}                                                                                                                       \
	} while (false)

#define AUTORTFM_ENSURE_MSG_V(Condition, Format, Args)                                                                  \
	do                                                                                                                  \
	{                                                                                                                   \
		if (AUTORTFM_UNLIKELY(!(Condition)))                                                                            \
		{                                                                                                               \
			[[maybe_unused]] static bool bCalled = [&]                                                                  \
			{                                                                                                           \
				::AutoRTFM::EnsureFailureV(__FILE__, __LINE__, /* ProgramCounter */ nullptr, #Condition, Format, Args); \
				return true;                                                                                            \
			}();                                                                                                        \
		}                                                                                                               \
	} while (false)

#define AUTORTFM_DUMP_STACKTRACE(Message, ...) ::AutoRTFM::LogWithCallstack(autortfm_log_error, Message, ##__VA_ARGS__)

// This is all a bit funky - but for good reason! We want `AUTORTFM_ASSERT`
// to be as close to *zero* cost if the assert wouldn't trigger.
// We will always pay the cost of the unlikely branch, but we need
// to make the body of taking the assert happen in another function
// so as not to affect codegen, register allocation, and stack
// reservation. But we also want the `AUTORTFM_ASSERT` to give us accurate
// `__FILE__` and `__LINE__` information for the line that it was
// triggered on. So what we do is have a lambda at the line that
// actually does the assert (but crucially gets the correct file
// and line numbers), but then pass this to another function
// `DoAssert` which has a special calling convention that makes
// the caller as optimal as possible (at the expense of the callee).
#define AUTORTFM_ASSERT(exp)                                     \
	if (AUTORTFM_UNLIKELY(!(exp)))                               \
	{                                                            \
		auto Lambda = []                                         \
		{                                                        \
			AUTORTFM_FATAL("AUTORTFM_ASSERT(%s) failure", #exp); \
		};                                                       \
		AutoRTFM::DoAssert(Lambda);                              \
	}

// For places where zero cost is truly important, AUTORTFM_ASSERT_DEBUG will
// disappear entirely in Shipping and Test builds. We still evaluate the expression,
// on the off-chance that we are given an expression with side effects, and rely on
// the compiler to optimize away a dead expression.
#if AUTORTFM_BUILD_SHIPPING || AUTORTFM_BUILD_TEST
#define AUTORTFM_ASSERT_DEBUG(exp) ((void)AUTORTFM_LIKELY(static_cast<bool>(exp)))
#else
#define AUTORTFM_ASSERT_DEBUG(exp)                                     \
	if (AUTORTFM_UNLIKELY(!(exp)))                                     \
	{                                                                  \
		auto Lambda = []                                               \
		{                                                              \
			AUTORTFM_FATAL("AUTORTFM_ASSERT_DEBUG(%s) failure", #exp); \
		};                                                             \
		AutoRTFM::DoAssert(Lambda);                                    \
	}
#endif

// Same funkiness as AUTORTFM_ASSERT, except that it doesn't cause a fatal error.
// TODO: Maybe upgrade the Info -> Warning?
#define AUTORTFM_EXPECT(exp)                                                                      \
	if (AUTORTFM_UNLIKELY(!(exp)))                                                                \
	{                                                                                             \
		auto Lambda = []                                                                          \
		{                                                                                         \
			::AutoRTFM::LogWithCallstack(autortfm_log_info, "AUTORTFM_EXPECT(%s) failure", #exp); \
		};                                                                                        \
		AutoRTFM::DoExpect(Lambda);                                                               \
	}

#if defined(__has_feature)
#if __has_feature(address_sanitizer)
#define AUTORTFM_NO_ASAN [[clang::no_sanitize("address")]]
#endif
#endif

#if !defined(AUTORTFM_NO_ASAN)
#define AUTORTFM_NO_ASAN
#endif

#if defined(__clang__)
#define AUTORTFM_MUST_TAIL [[clang::musttail]]
#endif

#if !defined(AUTORTFM_MUST_TAIL)
#define AUTORTFM_MUST_TAIL
#endif

#if !defined(CA_SUPPRESS)
#define CA_SUPPRESS(WarningNumber)
#endif

[[noreturn]] inline void AutoRTFM::InternalUnreachable()
{
	AUTORTFM_FATAL("Unreachable encountered!");
	__builtin_unreachable();
}

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
