// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuildMacros.h"

#if AUTORTFM_PLATFORM_WINDOWS
#define WIN32_LEAN_AND_MEAN
#define NOMINMAX
#include <Windows.h>
#include <tchar.h>
#endif  // AUTORTFM_PLATFORM_WINDOWS
