// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (defined(__AUTORTFM) && __AUTORTFM)

#include "Allocator.h"
#include "AutoRTFM.h"
#include "BuildMacros.h"
#include "Utils.h"

#include <algorithm>
#include <cstddef>
#include <cstdint>
#include <cstring>
#include <stdint.h>

namespace AutoRTFM
{
struct AUTORTFM_INTERNAL FWriteLogEntry final
{
	// The address of the write.
	std::byte* LogicalAddress = nullptr;

	// A pointer to the original data before the write occurred.
	std::byte* Data = nullptr;

	// The size of the write in bytes.
	size_t Size = 0;

	// Flags on the write.
	EWriteFlags Flags = EWriteFlags::Default;

	// Returns a new FWriteLogEntry with the logical address and data
	// pointers increased by NumBytes, the size decreased by NumBytes, and
	// the same flags.
	[[nodiscard]] FWriteLogEntry Advance(size_t NumBytes) const
	{
		return {
			.LogicalAddress = LogicalAddress + NumBytes,
			.Data = Data + NumBytes,
			.Size = Size - NumBytes,
			.Flags = Flags,
		};
	}
};

// FWriteLog holds an ordered list of write records which can be iterated
// forwards and backwards.
// Ensure changes to this class are kept in sync with AutoRTFM.natvis.
class AUTORTFM_INTERNAL FWriteLog final
{
	enum EAppendMode
	{
		Partial,
		Whole,
	};

public:
	template <size_t SizeBits, size_t FlagBits, size_t AddressBits>
	struct FRecordLayout
	{
		// Number of bits used by the FRecord to represent a write's address.
		static constexpr size_t NumAddressBits = AddressBits;

		// Number of bits used by the FRecord to represent a write's flags.
		static constexpr size_t NumFlagsBits = FlagBits;

		// Number of bits used by the FRecord to represent a write's size.
		// Note a record size must be greater than one byte, so is stored as Size-1.
		static constexpr size_t NumSizeBits = SizeBits;

		static_assert(NumAddressBits + NumFlagsBits + NumSizeBits == 64);

		static constexpr size_t AddressShift = 0;
		static constexpr size_t AddressMask = (static_cast<size_t>(1) << NumAddressBits) - 1;
		static constexpr size_t FlagsShift = NumAddressBits;
		static constexpr size_t FlagsMask = (static_cast<size_t>(1) << NumFlagsBits) - 1;
		static constexpr size_t SizeShift = NumAddressBits + NumFlagsBits;
		static constexpr size_t SizeMask = (static_cast<size_t>(1) << NumSizeBits) - 1;

		static constexpr size_t MaxAddress = AddressMask;
		static constexpr size_t MaxSize = SizeMask + 1;

		[[nodiscard]] static uint64_t Pack(std::byte* Address, EWriteFlags Flags, size_t Size)
		{
			const size_t SizeM1 = Size - 1;
			AUTORTFM_ASSERT_DEBUG((static_cast<size_t>(reinterpret_cast<uintptr_t>(Address)) & ~AddressMask) == 0);
			AUTORTFM_ASSERT_DEBUG((static_cast<size_t>(Flags) & ~FlagsMask) == 0);
			AUTORTFM_ASSERT_DEBUG((SizeM1 & ~SizeMask) == 0);

			return reinterpret_cast<uint64_t>(Address) | (static_cast<uint64_t>(Flags) << FlagsShift)
				 | (static_cast<uint64_t>(SizeM1) << SizeShift);
		}

		[[nodiscard]] static uint64_t ReplaceSize(uint64_t Packed, size_t NewSize)
		{
			const size_t NewSizeM1 = NewSize - 1;
			AUTORTFM_ASSERT_DEBUG((NewSizeM1 & ~SizeMask) == 0);
			constexpr uint64_t Mask = ~(SizeMask << SizeShift);
			return (Packed & Mask) | (NewSizeM1 << SizeShift);
		}

		[[nodiscard]] static std::byte* UnpackAddress(uint64_t Packed)
		{
			return reinterpret_cast<std::byte*>((Packed >> AddressShift) & AddressMask);
		}

		[[nodiscard]] static EWriteFlags UnpackFlags(uint64_t Packed)
		{
			return static_cast<EWriteFlags>((Packed >> NumAddressBits) & FlagsMask);
		}

		[[nodiscard]] static size_t UnpackSize(uint64_t Packed)
		{
			const size_t SizeM1 = reinterpret_cast<size_t>((Packed >> SizeShift) & SizeMask);
			return SizeM1 + 1;
		}
	};

	//  64           50 48                                              0
	//  ┌─────────────┬─┬───────────────────────────────────────────────┐
	//  │    Size     │F│                    Address                    │
	//  └─────────────┴─┴───────────────────────────────────────────────┘
	using FRegularRecordLayout = FRecordLayout</* SizeBits */ 14, /* FlagBits */ 2, /* AddressBits */ 48>;

	//  64         52   48                                              0
	//  ┌───────────┬───┬───────────────────────────────────────────────┐
	//  │   Size    │ F │                    Address                    │
	//  └───────────┴───┴───────────────────────────────────────────────┘
	using FCustomRollbackRecordLayout = FRecordLayout</* SizeBits */ 12, /* FlagBits */ 4, /* AddressBits */ 48>;

private:
	struct FRecord
	{
		// Returns true if the record has the EWriteFlags::CustomRollback flag set.
		// This alters the bit packing layout of the record.
		bool UsesCustomRollback() const
		{
			return (FRegularRecordLayout::UnpackFlags(Bits) & EWriteFlags::CustomRollback) == EWriteFlags::CustomRollback;
		}

		// Packs the address, flags and size into the record using the provided bit packing layout.
		template <typename Layout>
		void Set(std::byte* Address, EWriteFlags Flags, size_t Size)
		{
			AUTORTFM_ASSERT_DEBUG(Size > 0);
			Bits = Layout::Pack(Address, Flags, Size);
		}

		// Replaces the size field of the record using the provided bit packing layout.
		template <typename Layout>
		void SetSize(size_t NewSize)
		{
			AUTORTFM_ASSERT_DEBUG(NewSize > 0);
			Bits = Layout::ReplaceSize(Bits, NewSize);
		}

		// Returns the unpacked address from the record.
		std::byte* Address() const
		{
			return UsesCustomRollback() ? FCustomRollbackRecordLayout::UnpackAddress(Bits)
										: FRegularRecordLayout::UnpackAddress(Bits);
		}

		// Returns the unpacked flags from the record.
		EWriteFlags Flags() const
		{
			return UsesCustomRollback() ? FCustomRollbackRecordLayout::UnpackFlags(Bits) : FRegularRecordLayout::UnpackFlags(Bits);
		}

		// Returns the unpacked size from the record.
		size_t Size() const
		{
			return UsesCustomRollback() ? FCustomRollbackRecordLayout::UnpackSize(Bits) : FRegularRecordLayout::UnpackSize(Bits);
		}

		// Returns the unpacked size from the record, using the bit packing layout.
		template <typename Layout>
		size_t Size() const
		{
			return Layout::UnpackSize(Bits);
		}

		uint64_t Bits;
	};

	static_assert(sizeof(uintptr_t) == 8, "assumption: a pointer is 8 bytes");
	static_assert(sizeof(FRecord) == 8);

	// Ensure changes to this structure are kept in sync with Unreal.natvis.
	struct FBlock final
	{
		// ┌────────┬────┬────┬────┬────┬────────────────┬────┬────┬────┬────┐
		// │ FBlock │ D₀ │ D₁ │ D₂ │ D₃ │->            <-│ R₃ │ R₂ │ R₁ │ R₀ │
		// └────────┴────┴────┴────┴────┴────────────────┴────┴────┴────┴────┘
		//          ^                   ^                ^              ^
		//      DataStart()          DataEnd         LastRecord    FirstRecord
		// Where:
		//   Dₙ = Data n, Rₙ = Record n

		// Starting size of a heap-allocated block, including the FBlock struct header.
		static constexpr size_t DefaultSize = 2048;

		// Constructor
		// TotalSize is the total size of the allocated memory for the block including
		// the FBlock header.
		explicit FBlock(size_t TotalSize)
		{
			AUTORTFM_ENSURE((TotalSize & (alignof(FRecord) - 1)) == 0);
			std::byte* End = reinterpret_cast<std::byte*>(this) + TotalSize;
			DataEnd = DataStart();
			// Note: The initial empty state has LastRecord pointing one
			// FRecord beyond the immutable FirstRecord.
			LastRecord = reinterpret_cast<FRecord*>(End);
			FirstRecord = LastRecord - 1;
		}

		// Allocate performs a heap allocation of a new block.
		// TotalSize is the total size of the allocated memory for the block including
		// the FBlock header.
		static FBlock* Allocate(size_t TotalSize)
		{
			AUTORTFM_ASSERT(TotalSize > (sizeof(FBlock) + sizeof(FRecord)));
			void* Memory = FAllocator::Allocate(TotalSize, alignof(FBlock));
			// Disable false-positive warning C6386: Buffer overrun while writing to 'Memory'
			CA_SUPPRESS(6386)
			return new (Memory) FBlock(TotalSize);
		}

		// Free releases the heap-allocated memory for this block.
		// Note: This block must have been allocated with a call to Allocate().
		void Free()
		{
			std::byte* const End = reinterpret_cast<std::byte*>(FirstRecord + 1);
			size_t AllocationSize = static_cast<size_t>(End - reinterpret_cast<std::byte*>(this));
			FAllocator::Free(this, AllocationSize);
		}

		// Returns a pointer to the data for the first entry
		std::byte* DataStart()
		{
			return reinterpret_cast<std::byte*>(this) + sizeof(FBlock);
		}

		// Returns a pointer to the data for the last entry
		std::byte* LastData()
		{
			return DataEnd - LastRecord->Size();
		}

		// Returns true if the block holds no entries.
		bool IsEmpty() const
		{
			return LastRecord > FirstRecord;
		}

		// Returns the number of unused bytes in the block between the last
		// record and end of the data.
		UE_AUTORTFM_FORCEINLINE size_t SpaceRemaining() const
		{
			return reinterpret_cast<std::byte*>(LastRecord) - DataEnd;
		}

		// Appends a new record to the block, returning the number of bytes
		// copied to the new record from the front of DataIn.
		//
		// If AppendMode is Partial, then AppendWithNewRecord() will append
		// as many bytes that will fit in the remaining available space of
		// the block, which may be fewer than NumBytes. In this mode
		// NumBytes may be 0, in which case the function does nothing and
		// returns 0.
		//
		// If AppendMode is Whole, then it is the caller's responsibility
		// to ensure that there is enough space in the block to hold an
		// additional NumBytes of data plus the size of an FRecord. In this
		// mode NumBytes must be greater than zero.
		//
		// Layout is the bit-packing layout of a record with the given flags.
		template <EAppendMode AppendMode, typename Layout>
		UE_AUTORTFM_FORCEINLINE size_t AppendWithNewRecord(
			std::byte* LogicalAddress, std::byte* DataIn, size_t NumBytes, EWriteFlags Flags)
		{
			if constexpr (AppendMode == Partial)
			{
				const size_t Space = SpaceRemaining();
				if (NumBytes == 0 || Space <= sizeof(FRecord))
				{
					return 0;
				}
				NumBytes = std::min(std::min(NumBytes, Space - sizeof(FRecord)), Layout::MaxSize);
			}
			else
			{
				static_assert(AppendMode == Whole);
				AUTORTFM_ASSERT_DEBUG(NumBytes > 0);
			}

			AUTORTFM_ASSERT_DEBUG(SpaceRemaining() >= sizeof(FRecord) + NumBytes);

			LastRecord--;
			LastRecord->Set<Layout>(LogicalAddress, Flags, NumBytes);
			memcpy(DataEnd, DataIn, NumBytes);
			DataEnd += NumBytes;
			AUTORTFM_ASSERT_DEBUG(DataEnd <= reinterpret_cast<std::byte*>(LastRecord));

			return NumBytes;
		}

		// Appends additional data to the tail record, returning the number
		// of bytes copied to the end of the record from the front of DataIn.
		//
		// Regardless of AppendMode, if the last record cannot be appended
		// to, either because the logical address does not start at the end
		// of the last record's logical address or the flags are
		// incompatible, then the function returns 0.
		//
		// If AppendMode is Partial, then AppendWithNewRecord() will append
		// as many bytes that will fit in the remaining available space of
		// the block, which may be fewer than NumBytes.
		//
		// If AppendMode is Whole, then AppendWithNewRecord() will only
		// append the full NumBytes if they fit, otherwise the function will
		// do nothing and return 0.
		//
		// Layout is the bit-packing layout of a record with the given flags.
		template <EAppendMode AppendMode, typename Layout>
		UE_AUTORTFM_FORCEINLINE size_t AppendToLastRecord(
			std::byte* LogicalAddress, std::byte* DataIn, size_t NumBytes, EWriteFlags Flags)
		{
			if (IsEmpty())
			{
				return 0;
			}

			const size_t LastRecordSize = LastRecord->Size();
			if (LogicalAddress != LastRecord->Address() + LastRecordSize || LastRecord->Flags() != Flags)
			{
				return 0;
			}

			const size_t RepresentableSizeRemaining = Layout::MaxSize - LastRecordSize;
			const size_t Available = std::min(RepresentableSizeRemaining, SpaceRemaining());

			if constexpr (AppendMode == Whole)
			{
				if (NumBytes > Available)
				{
					return 0;
				}
			}
			else
			{
				static_assert(AppendMode == Partial);
				NumBytes = std::min(NumBytes, Available);
			}

			LastRecord->SetSize<Layout>(LastRecordSize + NumBytes);
			memcpy(DataEnd, DataIn, NumBytes);
			DataEnd += NumBytes;
			AUTORTFM_ASSERT_DEBUG(DataEnd <= reinterpret_cast<std::byte*>(LastRecord));
			return NumBytes;
		}

		// The next block in the linked list.
		FBlock* NextBlock = nullptr;
		// The previous block in the linked list.
		FBlock* PrevBlock = nullptr;
		// The pointer to the first entry's record
		FRecord* FirstRecord = nullptr;
		// The pointer to the last entry's record
		FRecord* LastRecord = nullptr;
		// One byte beyond the end of the last entry's data
		std::byte* DataEnd = nullptr;

	private:
		~FBlock() = delete;
	};

public:
	// Constructor
	FWriteLog()
	{
		new (HeadBlockMemory) FBlock(HeadBlockSize);
	}

	// Destructor
	~FWriteLog()
	{
		Reset();
	}

	// Adds the write log entry to the log.
	// The log will make a copy of the FWriteLogEntry's data.
	void Push(FWriteLogEntry Entry)
	{
		if ((Entry.Flags & EWriteFlags::CustomRollback) == EWriteFlags::CustomRollback)
		{
			Push<FCustomRollbackRecordLayout>(Entry);
		}
		else
		{
			Push<FRegularRecordLayout>(Entry);
		}
	}

	// Adds the write log entry to the log; assumes a payload small enough that splitting is not beneficial.
	// If you have large sizes, you should use `Push` instead so that splitting can occur.
	template <unsigned int Size>
	void PushSmall(std::byte* LogicalAddress, std::byte* Data, EWriteFlags Flags = EWriteFlags::Default)
	{
		if ((Flags & EWriteFlags::CustomRollback) == EWriteFlags::CustomRollback)
		{
			PushSmall<Size, FCustomRollbackRecordLayout>(LogicalAddress, Data, Flags);
		}
		else
		{
			PushSmall<Size, FRegularRecordLayout>(LogicalAddress, Data, Flags);
		}
	}

	// Iterator for enumerating the writes of the log.
	template <bool IS_FORWARD>
	struct TIterator final
	{
		TIterator() = default;

		TIterator(FBlock* StartBlock) : Block(StartBlock)
		{
			if (UNLIKELY(Block->IsEmpty()))
			{
				if (UNLIKELY(!AdvanceBlock()))
				{
					// The write log is entirely empty.
					return;
				}
			}

			Data = IS_FORWARD ? Block->DataStart() : Block->LastData();
			Record = IS_FORWARD ? Block->FirstRecord : Block->LastRecord;
		}

		// Returns the entry at the current iterator's position.
		FWriteLogEntry operator*() const
		{
			return FWriteLogEntry{
				.LogicalAddress = reinterpret_cast<std::byte*>(Record->Address()),
				.Data = Data,
				.Size = Record->Size(),
				.Flags = Record->Flags(),
			};
		}

		// Progresses the iterator to the next entry
		void operator++()
		{
			if constexpr (IS_FORWARD)
			{
				if (Record == Block->LastRecord)
				{
					if (LIKELY(AdvanceBlock()))
					{
						Data = Block->DataStart();
						Record = Block->FirstRecord;
					}
				}
				else
				{
					Data += Record->Size();
					Record--;
				}
			}
			else
			{
				if (Record == Block->FirstRecord)
				{
					if (LIKELY(AdvanceBlock()))
					{
						Data = Block->LastData();
						Record = Block->LastRecord;
					}
				}
				else
				{
					Record++;
					Data -= Record->Size();
				}
			}
		}

		// Inequality operator
		bool operator!=(const TIterator& Other) const
		{
			return (Other.Block != Block) || (Other.Record != Record);
		}

	private:
		// Resets the iterator (compares equal to the write log's end())
		UE_AUTORTFM_FORCEINLINE void Reset()
		{
			Block = nullptr;
			Data = nullptr;
			Record = nullptr;
		}

		// Moves from this block to the next (if IS_FORWARD) or previous (if not IS_FORWARD),
		// skipping any empty blocks. Returns true on success. If no more blocks exist, resets
		// the iterator and returns false.
		UE_AUTORTFM_FORCEINLINE bool AdvanceBlock()
		{
			do
			{
				Block = IS_FORWARD ? Block->NextBlock : Block->PrevBlock;
			} while (Block && Block->IsEmpty());

			if (!Block)
			{
				Reset();
				return false;
			}

			return true;
		}

		FBlock* Block = nullptr;
		std::byte* Data = nullptr;
		FRecord* Record = nullptr;
	};

	using Iterator = TIterator</* IS_FORWARD */ true>;
	using ReverseIterator = TIterator</* IS_FORWARD */ false>;

	Iterator begin() const
	{
		return (NumEntries > 0) ? Iterator(HeadBlock) : Iterator{};
	}
	ReverseIterator rbegin() const
	{
		return (NumEntries > 0) ? ReverseIterator(TailBlock) : ReverseIterator{};
	}
	Iterator end() const
	{
		return Iterator{};
	}
	ReverseIterator rend() const
	{
		return ReverseIterator{};
	}

	// Resets the write log to its initial state, freeing any allocated memory.
	void Reset()
	{
		// Skip HeadBlock, which is held as part of this structure.
		FBlock* Block = HeadBlock->NextBlock;
		while (nullptr != Block)
		{
			FBlock* const Next = Block->NextBlock;
			Block->Free();
			Block = Next;
		}
		new (HeadBlockMemory) FBlock(HeadBlockSize);
		HeadBlock = reinterpret_cast<FBlock*>(HeadBlockMemory);
		TailBlock = reinterpret_cast<FBlock*>(HeadBlockMemory);
		NumEntries = 0;
		TotalSizeBytes = 0;
		NextBlockSize = FBlock::DefaultSize;
	}

	// Returns true if the log holds no entries.
	UE_AUTORTFM_FORCEINLINE bool IsEmpty() const
	{
		return 0 == NumEntries;
	}

	// Return the number of entries in the log.
	UE_AUTORTFM_FORCEINLINE size_t Num() const
	{
		return NumEntries;
	}

	// Return the total size in bytes for all entries in the log.
	UE_AUTORTFM_FORCEINLINE size_t TotalSize() const
	{
		return TotalSizeBytes;
	}

private:
	template <typename Layout>
	void Push(FWriteLogEntry Entry)
	{
		TotalSizeBytes += Entry.Size;

		// Attempt to append to the end of the last record.
		if (const size_t Appended =
				TailBlock->AppendToLastRecord<Partial, Layout>(Entry.LogicalAddress, Entry.Data, Entry.Size, Entry.Flags))
		{
			Entry = Entry.Advance(Appended);
		}

		// Append to the current tail block with new records until we've
		// either filled the block, or run out of data to append.
		while (const size_t Appended =
				   TailBlock->AppendWithNewRecord<Partial, Layout>(Entry.LogicalAddress, Entry.Data, Entry.Size, Entry.Flags))
		{
			NumEntries++;
			Entry = Entry.Advance(Appended);
		}

		if (AUTORTFM_LIKELY(Entry.Size == 0))
		{
			return;
		}

		// TailBlock is full, and there's more data to append.
		// We need to allocate a new block.

		// The maximum number of bytes that can be stored in a single record.
		const size_t RecordMaxSize = Layout::MaxSize;
		// Calculate how many records we need to make.
		const size_t NumRecords = (Entry.Size + RecordMaxSize - 1) / RecordMaxSize;
		// Calculate the exact required size of the block.
		const size_t RequiredSize = sizeof(FBlock) +                // FBlock header
									NumRecords * sizeof(FRecord) +  // Bytes needed for the records
									Entry.Size;                     // Bytes needed for the data

		// Create a new empty tail block, large enough to hold the remainder
		// of this push in its entirety, and never smaller than the upcoming
		// block size.
		AllocateNewBlock(std::max(RequiredSize, NextBlockSize));

		// Append (NumRecords-1) full-records.
		for (size_t I = 1; I < NumRecords; I++)
		{
			TailBlock->AppendWithNewRecord<Whole, Layout>(Entry.LogicalAddress, Entry.Data, RecordMaxSize, Entry.Flags);
			NumEntries++;
			Entry = Entry.Advance(RecordMaxSize);
		}

		// Append last partial record.
		if (Entry.Size)
		{
			TailBlock->AppendWithNewRecord<Whole, Layout>(Entry.LogicalAddress, Entry.Data, Entry.Size, Entry.Flags);
			NumEntries++;
		}
	}

	template <size_t Size, typename Layout>
	void PushSmall(std::byte* LogicalAddress, std::byte* Data, EWriteFlags Flags)
	{
		static_assert(Size <= Layout::MaxSize);

		TotalSizeBytes += Size;

		if (TailBlock->AppendToLastRecord<Whole, Layout>(LogicalAddress, Data, Size, Flags))
		{
			return;
		}

		if (TailBlock->SpaceRemaining() < sizeof(FRecord) + Size)
		{
			AllocateNewBlock(NextBlockSize);
		}

		TailBlock->AppendWithNewRecord<Whole, Layout>(LogicalAddress, Data, Size, Flags);
		NumEntries++;
	}

	UE_AUTORTFM_FORCEINLINE void AllocateNewBlock(size_t Size)
	{
		FBlock* NewBlock = FBlock::Allocate(AlignUp(Size, sizeof(FRecord)));
		NewBlock->PrevBlock = TailBlock;
		TailBlock->NextBlock = NewBlock;
		TailBlock = NewBlock;

		// Increase block sizes by 50% each time, capped at MaxBlockSize.
		NextBlockSize = std::min<size_t>((NextBlockSize * 3 / 2), MaxBlockSize);
	}

	template <size_t SIZE>
	static constexpr bool IsAlignedForFRecord = (SIZE & (alignof(FRecord) - 1)) == 0;

	// The size of the inline block, which is declared as a byte array (`HeadBlockMemory`)
	// inside the write log.
	static constexpr size_t HeadBlockSize = 256;

	// The upper bound on heap-allocated block size.
	static constexpr size_t MaxBlockDataSize = 1024 * 1024 * 16;  // Without header, record or alignment.
	static constexpr size_t MaxBlockSize = AlignUp(sizeof(FBlock) + sizeof(FRecord) + MaxBlockDataSize, alignof(FRecord));

	static_assert(IsAlignedForFRecord<HeadBlockSize>);
	static_assert(IsAlignedForFRecord<FBlock::DefaultSize>);
	static_assert(IsAlignedForFRecord<MaxBlockSize>);

	FBlock* HeadBlock = reinterpret_cast<FBlock*>(HeadBlockMemory);
	FBlock* TailBlock = reinterpret_cast<FBlock*>(HeadBlockMemory);
	size_t NumEntries = 0;
	size_t TotalSizeBytes = 0;
	size_t NextBlockSize = FBlock::DefaultSize;
	alignas(alignof(FBlock)) std::byte HeadBlockMemory[HeadBlockSize];
};
}  // namespace AutoRTFM

#endif  // (defined(__AUTORTFM) && __AUTORTFM)
