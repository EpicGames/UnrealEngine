# Copyright Epic Games, Inc. All Rights Reserved.

# AutoRTFM external mappings for ASAN.
# See Engine/Source/Runtime/AutoRTFM/Documentation/ExternalMappings.md
# for syntax and more information.

same __asan_addr_is_in_fake_stack
same __asan_get_current_fake_stack
same __asan_poison_memory_region
