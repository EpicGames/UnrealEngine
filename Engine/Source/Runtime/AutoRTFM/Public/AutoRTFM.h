// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 *  AutoRTFM is designed to make it easy to take existing C++ code--even if it was never designed
 *  to have any transactional semantics--and make it transactional just by using an alternate compiler.
 *  For details, see `Engine/Source/Runtime/AutoRTFM/Documentation/README.md`.
 */

// HEADER_UNIT_SKIP - unused warnings

#include "AutoRTFM/CAPI.h"
#include "AutoRTFM/Constants.h"
#include "AutoRTFM/Defines.h"
#include "AutoRTFM/ReturnFromOpenMode.h"

#ifdef __cplusplus

#include "AutoRTFM/Task.h"

#include <cstddef>
#include <new>
#include <type_traits>
#include <utility>

namespace AutoRTFM
{

enum class EWriteFlags
{
	// No flags applied to the write.
	Default = autortfm_write_default,
	// The write should not be considered by the AutoRTFM sanitizer.
	NoSanitize = autortfm_write_no_sanitize,
	// The autortfm_extern_api::RollbackWrite will be called to perform
	// the rollback for this write if the transaction is aborted.
	CustomRollback = autortfm_write_custom_rollback,
	// Custom flag bits which can be used by the custom abort callbacks.
	// Requires CustomRollback for usage.
	CustomFlag0 = autortfm_write_custom_flag_0,
	CustomFlag1 = autortfm_write_custom_flag_1,
};

// Bitwise operator | for EWriteFlags
inline constexpr EWriteFlags operator|(EWriteFlags LHS, EWriteFlags RHS)
{
	return static_cast<EWriteFlags>(static_cast<int>(LHS) | static_cast<int>(RHS));
}
// Bitwise operator & for EWriteFlags
inline constexpr EWriteFlags operator&(EWriteFlags LHS, EWriteFlags RHS)
{
	return static_cast<EWriteFlags>(static_cast<int>(LHS) & static_cast<int>(RHS));
}

// Evaluates to the AutoRTFM mode of the function overload which would be called
// when calling FuncType with the arguments of type ArgTypes.
// Warning: This is an experimental API and may be removed in the future.
template <typename FuncType, typename... ArgTypes>
static constexpr autortfm_mode CallMode = AUTORTFM_MODE_OF_CALL(std::declval<FuncType>()(std::declval<ArgTypes>()...));

// Evaluates to the AutoRTFM mode of the destructor function of the type Type.
// Warning: This is an experimental API and may be removed in the future.
template <typename Type>
static constexpr autortfm_mode DestructorMode = AUTORTFM_MODE_OF_CALL(std::declval<Type>().~Type());

// Evaluates to the AutoRTFM mode of the constructor function of the type Type
// when called with the arguments of type ArgTypes.
// Warning: This is an experimental API and may be removed in the future.
template <typename Type, typename... ArgTypes>
static constexpr autortfm_mode ConstructorMode = AUTORTFM_MODE_OF_CALL(::new Type(std::declval<ArgTypes>()...));

// The transaction result provides information on how a transaction completed.
enum class ETransactionResult
{
	// The transaction committed successfully. For a nested transaction this does not mean that the transaction effects
	// cannot be undone later if the parent transaction is aborted for any reason.
	Committed = autortfm_transaction_committed,

	// The transaction aborted because of an explicit call to AbortTransaction.
	AbortedByRequest = autortfm_transaction_aborted_by_request,

	// The transaction aborted because of unhandled constructs in the code (atomics, unhandled function calls, etc).
	AbortedByLanguage = autortfm_transaction_aborted_by_language,

	// The transaction aborted because of an explicit call to CascadingAbortTransaction.
	AbortedByCascadingAbort = autortfm_transaction_aborted_by_cascading_abort,

	// A transaction was rejected because a new transaction nest was attempted
	// after a cascading abort or cascading retry.
	RejectedTransactDuringUnwind = autortfm_transaction_rejected_during_unwind,

	// A transaction was rejected because a new transaction nest was attempted
	// (via OnCommit) while the current transaction was being committed.
	RejectedTransactDuringCommit = autortfm_transaction_rejected_during_commit,

	// A transaction was rejected because a new transaction nest was attempted
	// (via OnAbort) while the current transaction was being aborted.
	RejectedTransactDuringAbort = autortfm_transaction_rejected_during_abort,

	// A transaction was rejected because a new transaction nest was attempted
	// (via OnRetry) while the current transaction was being retried.
	RejectedTransactDuringRetry = autortfm_transaction_rejected_during_retry,

	// A transaction was rejected because a new transaction nest was attempted
	// (via OnComplete) while the current transaction was being retried.
	RejectedTransactDuringCompletion = autortfm_transaction_rejected_during_complete,
};

// Returns true if the given ETransactionResult represents a transaction that
// has aborted.
inline bool HasAborted(ETransactionResult State)
{
	// Note: Intentionally verbose so that the compiler will error if new enumerators are added.
	switch (State)
	{
		case ETransactionResult::Committed:
			return false;

		case ETransactionResult::AbortedByRequest:
		case ETransactionResult::AbortedByLanguage:
		case ETransactionResult::AbortedByCascadingAbort:
		case ETransactionResult::RejectedTransactDuringUnwind:
		case ETransactionResult::RejectedTransactDuringCommit:
		case ETransactionResult::RejectedTransactDuringAbort:
		case ETransactionResult::RejectedTransactDuringRetry:
		case ETransactionResult::RejectedTransactDuringCompletion:
			break;
	}
	return true;
}

// The transaction status provides information on how a transaction is currently executing or completed.
enum class ETransactionStatus
{
	// The transaction aborted because of an explicit call to AbortTransaction.
	Executing = autortfm_transaction_executing,

	// The transaction committed successfully. For a nested transaction this does not mean that the transaction effects
	// cannot be undone later if the parent transaction is aborted for any reason.
	Committed = autortfm_transaction_committed,

	// The transaction aborted because of an explicit call to AbortTransaction.
	AbortedByRequest = autortfm_transaction_aborted_by_request,

	// The transaction aborted because of unhandled constructs in the code (atomics, unhandled function calls, etc).
	AbortedByLanguage = autortfm_transaction_aborted_by_language,

	// The transaction aborted because of an explicit call to CascadingAbortTransaction.
	AbortedByCascadingAbort = autortfm_transaction_aborted_by_cascading_abort,

	// The transaction aborted because of an explicit call to CascadingRetryTransaction.
	AbortedByCascadingRetry = autortfm_transaction_aborted_by_cascading_retry,
};

// Returns true if the given ETransactionStatus represents a transaction that has aborted.
inline bool HasAborted(ETransactionStatus State)
{
	// Note: Intentionally verbose so that the compiler will error if new enumerators are added.
	switch (State)
	{
		case ETransactionStatus::Executing:
		case ETransactionStatus::Committed:
			return false;

		case ETransactionStatus::AbortedByRequest:
		case ETransactionStatus::AbortedByLanguage:
		case ETransactionStatus::AbortedByCascadingAbort:
		case ETransactionStatus::AbortedByCascadingRetry:
			break;
	}
	return true;
}

// Returns true if Status is AbortedByCascadingAbort or AbortedByCascadingRetry.
inline bool IsCascading(ETransactionStatus Status)
{
	// Note: Intentionally verbose so that the compiler will error if new enumerators are added.
	switch (Status)
	{
		case ETransactionStatus::AbortedByCascadingAbort:
		case ETransactionStatus::AbortedByCascadingRetry:
			return true;

		case ETransactionStatus::Executing:
		case ETransactionStatus::Committed:
		case ETransactionStatus::AbortedByRequest:
		case ETransactionStatus::AbortedByLanguage:
			break;
	}
	return false;
}

// The context status shows what state the AutoRTFM context is currently in.
enum class EContextStatus : uint8_t
{
	// An Idle status means we are not in transactional code - i.e. the
	// transaction stack is empty, or queried on a non-transactional thread.
	Idle = autortfm_status_idle,

	// An OnTrack status means we are in transactional code.
	OnTrack = autortfm_status_on_track,

	// An Unwinding status means we've performed a cascading abort / retry and
	// are now in the process of unwinding the stack to the outermost Transact()
	Unwinding = autortfm_status_unwinding,

	// We are currently running the on-commit handlers for a transaction.
	Committing = autortfm_status_committing,

	// We are currently running the on-abort handlers for a transaction.
	Aborting = autortfm_status_aborting,

	// We are currently running the on-retry handlers for a transaction stack.
	Retrying = autortfm_status_retrying,

	// We are currently running the on-complete handlers for a transaction stack.
	Completing = autortfm_status_completing,

	// Means we are in a static local initializer which always run in the open.
	// `IsTransactional()` will return `false` when in this state.
	InStaticLocalInitializer = autortfm_status_in_static_local_initializer,
};

// An opaque unique identifier for a transaction.
using TransactionID = autortfm_transaction_id;

#if AUTORTFM_SANITIZER

#define AUTORTFM_SANITIZER_INTERNAL [[clang::autortfm(autortfm_mode_internal)]]

// AutoRTFM sanitizer API (requires the -fsanitize=autortfm compiler flag)
namespace Sanitizer
{
// An enumerator of AutoRTFM sanitizer levels.
// The AutoRTFM sanitizer is used to detect modification by open-code to memory that was written by
// a transaction. In this situation, aborting the transaction can corrupt memory as the undo will
// overwrite the writes made in the open-code.
enum class EMode : uint8_t
{
	// AutoRTFM sanitizer is disabled.
	Disabled = autortfm_sanitizer_disabled,

	// AutoRTFM sanitizer is enabled and failures are treated as warnings.
	Warn = autortfm_sanitizer_warn,

	// AutoRTFM sanitizer is enabled and failures are treated as errors.
	Error = autortfm_sanitizer_error,
};

// Returns the global AutoRTFM sanitizer mode currently in use.
AUTORTFM_SANITIZER_INTERNAL UE_AUTORTFM_API EMode GetMode();

// Sets the global AutoRTFM sanitizer mode.
AUTORTFM_SANITIZER_INTERNAL UE_AUTORTFM_API void SetMode(EMode Mode);

// Returns true if the AutoRTFM sanitizer records the callstack of closed writes.
AUTORTFM_SANITIZER_INTERNAL UE_AUTORTFM_API bool RecordClosedWriteCallstacks();

// Sets whether the AutoRTFM sanitizer records the callstack of closed writes.
AUTORTFM_SANITIZER_INTERNAL UE_AUTORTFM_API void SetRecordClosedWriteCallstacks(bool bEnable);

// Opaque handle returned by BeginDisable()
using FDisableHandle = void*;

// Temporarily disables the AutoRTFM sanitizer for the current transaction
// and any child transactions.
// Must be paired with a call to EndDisable().
// Calls to BeginDisable() are internally counted and may be nested within
// other BeginDisable() / EndDisable() calls.
// This call is a no-op when called outside of a transaction.
AUTORTFM_SANITIZER_INTERNAL UE_AUTORTFM_API FDisableHandle BeginDisable();

// Ends temporarily disabling the AutoRTFM sanitizer.
// DisableHandle must be a handle returned by BeginDisable().
// See BeginDisable().
AUTORTFM_SANITIZER_INTERNAL UE_AUTORTFM_API void EndDisable(FDisableHandle);

// Returns true if there are any AutoRTFM transactions in flight.
// Can be safely called from any thread, although the result will be
// non-deterministic without synchronization between threads.
AUTORTFM_SANITIZER_INTERNAL UE_AUTORTFM_API bool AnyTransactionsInFlight();

// A helper class that temporarily disables the AutoRTFM sanitizer for
// the lifetime of the object.
class FDisableScope
{
public:
	UE_AUTORTFM_FORCEINLINE FDisableScope() : Handle{BeginDisable()} {}
	UE_AUTORTFM_FORCEINLINE ~FDisableScope()
	{
		EndDisable(Handle);
	}

private:
	FDisableScope(const FDisableScope&) = delete;
	FDisableScope& operator=(const FDisableScope&) = delete;
	FDisableHandle const Handle;
};
}  // namespace Sanitizer

#undef AUTORTFM_SANITIZER_INTERNAL

#define AUTORTFM_SANITIZER_DISABLE_SCOPE() \
	::AutoRTFM::Sanitizer::FDisableScope UE_AUTORTFM_CONCAT(AutoRTFMSanitizerModeScope, __COUNTER__)

#else  // ^^^ AUTORTFM_SANITIZER ^^^ | vvv !AUTORTFM_SANITIZER vvv

#define AUTORTFM_SANITIZER_DISABLE_SCOPE() static_assert(true) /* require semicolon */

#endif  // ^^^ !AUTORTFM_SANITIZER ^^^

#if UE_AUTORTFM
namespace ForTheRuntime
{
UE_AUTORTFM_API void OnCommitInternal(TTask<void()>&& Work);
UE_AUTORTFM_API void OnPreAbortInternal(TTask<void()>&& Work);
UE_AUTORTFM_API void OnAbortInternal(TTask<void()>&& Work);
UE_AUTORTFM_API void OnCompleteInternal(TTask<void()>&& Work);
UE_AUTORTFM_API void OnRetryInternal(TTask<void()>&& Work);
UE_AUTORTFM_API void PushOnCommitHandlerInternal(const void* Key, TTask<void()>&& Work);
UE_AUTORTFM_API void PopOnCommitHandlerInternal(const void* Key);
UE_AUTORTFM_API void PopAllOnCommitHandlersInternal(const void* Key);
UE_AUTORTFM_API void PushOnAbortHandlerInternal(const void* Key, TTask<void()>&& Work);
UE_AUTORTFM_API void PopOnAbortHandlerInternal(const void* Key);
UE_AUTORTFM_API void PopAllOnAbortHandlersInternal(const void* Key);
UE_AUTORTFM_API void RegisterOnCommitFromTheOpen(TTask<void()>&& Work);
UE_AUTORTFM_API void RegisterOnAbortFromTheOpen(TTask<void()>&& Work);
UE_AUTORTFM_API void RedirectedLoad(
	uint32_t AddressSpace, void* DestPointer, uint64_t Size, uint64_t SourceAddress, bool bWillWriteHint);
UE_AUTORTFM_API void RedirectedLoad8(uint32_t AddressSpace, void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint);
UE_AUTORTFM_API void RedirectedLoad4(uint32_t AddressSpace, void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint);
UE_AUTORTFM_API void RedirectedLoad2(uint32_t AddressSpace, void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint);
UE_AUTORTFM_API void RedirectedLoad1(uint32_t AddressSpace, void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint);
UE_AUTORTFM_API void RedirectedStore(uint32_t AddressSpace, uint64_t DestAddress, uint64_t Size, const void* SourcePointer);
UE_AUTORTFM_API void RedirectedStore8(uint32_t AddressSpace, uint64_t DestAddress, const void* SourcePointer);
UE_AUTORTFM_API void RedirectedStore4(uint32_t AddressSpace, uint64_t DestAddress, const void* SourcePointer);
UE_AUTORTFM_API void RedirectedStore2(uint32_t AddressSpace, uint64_t DestAddress, const void* SourcePointer);
UE_AUTORTFM_API void RedirectedStore1(uint32_t AddressSpace, uint64_t DestAddress, const void* SourcePointer);
}  // namespace ForTheRuntime
#endif

template <typename FunctorType>
void AutoRTFMFunctorInvoker(void* Arg)
{
	(*static_cast<const FunctorType*>(Arg))();
}

#if UE_AUTORTFM
template <typename FunctorType>
auto AutoRTFMLookupInstrumentedFunctorInvoker(const FunctorType& Functor) -> void (*)(void*)
{
	// keep this as a single expression to help ensure that even Debug builds optimize this.
	// if we put intermediate results in local variables then the compiler emits loads
	// and stores to the stack which confuse our custom pass that tries to strip away
	// the actual call to autortfm_lookup_function
	void (*Result)(void*) = reinterpret_cast<void (*)(void*)>(autortfm_lookup_function(
		reinterpret_cast<void*>(&AutoRTFMFunctorInvoker<FunctorType>), "AutoRTFMLookupInstrumentedFunctorInvoker"));
	return Result;
}
#else
template <typename FunctorType>
auto AutoRTFMLookupInstrumentedFunctorInvoker(const FunctorType& Functor) -> void (*)(void*)
{
	return nullptr;
}
#endif

using FExternAPI = autortfm_extern_api;

// Initialize the AutoRTFM library.
// Must only be called once for the lifetime of the application.
#if UE_AUTORTFM
UE_AUTORTFM_API void Initialize(const FExternAPI& ExternAPI);
#else
UE_AUTORTFM_CRITICAL_INLINE void Initialize(const FExternAPI& ExternAPI)
{
	UE_AUTORTFM_UNUSED(ExternAPI);
}
#endif

// Shutdown the AutoRTFM library.
// Must only be called once for the lifetime of the application, after autortfm_initialize().
#if UE_AUTORTFM
UE_AUTORTFM_API void Shutdown();
#else
UE_AUTORTFM_CRITICAL_INLINE void Shutdown() {}
#endif

// Tells if we are currently running in the closed nest of a transaction. By default,
// transactional code is in a closed nest; the only way to be in an open nest is to request it
// via `Open`. This function is handled specially in the compiler and will be constant folded
// as true in closed code, and false in open code.
UE_AUTORTFM_CRITICAL_INLINE_ALWAYS bool IsClosed()
{
	return autortfm_is_closed();
}

// Returns the current AutoRTFM context status.
// Note: autortfm_get_context_status() is handled specially in the compiler and
// will be constant folded as EContextStatus::OnTrack when called in the closed.
UE_AUTORTFM_CRITICAL_INLINE EContextStatus ContextStatus()
{
	return static_cast<EContextStatus>(autortfm_get_context_status());
}

// Returns true if we are currently running in a transaction, and outside of an
// on-commit, on-abort, on-retry or on-complete handler.
// This will constant folded by the compiler to true when called in the closed.
UE_AUTORTFM_CRITICAL_INLINE_ALWAYS bool IsTransactional()
{
	return autortfm_is_context_status(autortfm_status_on_track);
}

// Returns true if we are currently outside of transactional code - i.e. the
// transaction stack is empty, or called from a non-transactional thread.
UE_AUTORTFM_CRITICAL_INLINE_ALWAYS bool IsIdle()
{
	return autortfm_is_context_status(autortfm_status_idle);
}

// Returns true if we are currently committing a transaction.
// This will return true when inside an on-commit handler.
UE_AUTORTFM_CRITICAL_INLINE_ALWAYS bool IsCommitting()
{
	return autortfm_is_context_status(autortfm_status_committing);
}

// Returns true if we are currently aborting a transaction.
// This will return true when inside an on-abort handler.
UE_AUTORTFM_CRITICAL_INLINE_ALWAYS bool IsAborting()
{
	return autortfm_is_context_status(autortfm_status_aborting);
}

// Returns true if we are currently retrying a transaction.
// This will return true when inside an on-retry handler.
UE_AUTORTFM_CRITICAL_INLINE_ALWAYS bool IsRetrying()
{
	return autortfm_is_context_status(autortfm_status_retrying);
}

// Returns true when inside an on-abort, on-commit, on-complete, on-retry handler,
// unwinding from a cascading abort / cascading retry, or inside a static initializer.
// TODO(SOL-8967): This function name is misleading. Rename to something sensible.
UE_AUTORTFM_CRITICAL_INLINE_ALWAYS bool IsCommittingOrAborting()
{
	switch (ContextStatus())
	{
		case AutoRTFM::EContextStatus::Idle:
		case AutoRTFM::EContextStatus::OnTrack:
			return false;
		case AutoRTFM::EContextStatus::Committing:
		case AutoRTFM::EContextStatus::Aborting:
		case AutoRTFM::EContextStatus::Completing:
		case AutoRTFM::EContextStatus::Retrying:
		case AutoRTFM::EContextStatus::InStaticLocalInitializer:
		case AutoRTFM::EContextStatus::Unwinding:
			break;
	}
	return true;
}

// Returns true if the passed-in pointer is on the stack of the currently-executing transaction.
// This is occasionally necessary when writing OnAbort handlers for objects on the stack, since
// we don't want to scribble on stack memory that might have been reused.
UE_AUTORTFM_CRITICAL_INLINE bool IsOnCurrentTransactionStack(void* Ptr)
{
	return autortfm_is_on_current_transaction_stack(Ptr);
}

// Returns an opaque identifier for the current transaction.
UE_AUTORTFM_CRITICAL_INLINE TransactionID CurrentTransactionID()
{
	return autortfm_current_transaction_id();
}

// Run the functor in a transaction. Memory writes and other side effects get instrumented
// and will be reversed if the transaction aborts.
//
// If this begins a nested transaction, the instrumented effects are logged onto the root
// transaction, so the effects can be reversed later if the root transaction aborts, even
// if this nested transaction succeeds.
//
// If AutoRTFM is disabled, the code will be ran non-transactionally.
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE ETransactionResult Transact(AUTORTFM_IMPLICIT_ENABLE const FunctorType& Functor)
{
#if UE_AUTORTFM_ENABLED
	static constexpr autortfm_mode FunctorAutoRTFMMode = CallMode<FunctorType>;
	static_assert(FunctorAutoRTFMMode == autortfm_mode_enable,
		"AutoRTFM mode of functor passed to AutoRTFM::Transact() is not autortfm_mode_enable");
#endif
	ETransactionResult Result = static_cast<ETransactionResult>(autortfm_transact(&AutoRTFMFunctorInvoker<FunctorType>,
		AutoRTFMLookupInstrumentedFunctorInvoker<FunctorType>(Functor), const_cast<void*>(static_cast<const void*>(&Functor))));

	return Result;
}

// This is just like calling Transact([&] { Open([&] { Functor(); }); });
// The reason we expose it is that it allows the caller's module to not
// be compiled with the AutoRTFM instrumentation of functions if the only
// thing that's being invoked is a function in the open.
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE ETransactionResult TransactThenOpen(const FunctorType& Functor)
{
	ETransactionResult Result = static_cast<ETransactionResult>(
		autortfm_transact_then_open(&AutoRTFMFunctorInvoker<FunctorType>, const_cast<void*>(static_cast<const void*>(&Functor))));

	return Result;
}

// Immediately aborts and pop the top-most transaction, discarding all effects.
// When called from the closed, control will jump to end of the next outer
// Transact() or Close().
// When called from the open, execution will continue until the next
// open -> closed transition, upon which, control will immediately jump to end
// of the next outer Transact() or Close().
UE_AUTORTFM_CRITICAL_INLINE void AbortTransaction()
{
	autortfm_abort_transaction();
}

// Immediately aborts and pops all transactions, discarding all effects.
// When called from the closed, control will jump to end of the next outer
// Transact() or Close().
// When called from the open, execution will continue until the next
// open -> closed transition, upon which, control will immediately jump to end
// of the next outer Transact() or Close().
// This process will repeat until the the outermost transaction is reached,
// upon which all registered on-complete callbacks will be invoked.
UE_AUTORTFM_CRITICAL_INLINE void CascadingAbortTransaction()
{
	autortfm_cascading_abort_transaction();
}

// Immediately aborts and pops all transactions, discarding all effects, then
// retries the outermost transaction.
// When called from the closed, control will jump to end of the next outer
// Transact() or Close().
// When called from the open, execution will continue until the next
// open -> closed transition, upon which, control will immediately jump to end
// of the next outer Transact() or Close().
// This process will repeat until the the outermost transaction is reached,
// upon which all registered on-retry callbacks will be invoked, then the
// outermost transaction will be restarted.
UE_AUTORTFM_CRITICAL_INLINE void CascadingRetryTransaction()
{
	autortfm_cascading_retry_transaction();
}

// Manually create a new transaction from open code and push it as a transaction nest.
// Can only be called within an already active parent transaction (EG. this cannot start
// a transaction nest itself).
// Can only be called from the open.
AUTORTFM_DISABLE UE_AUTORTFM_CRITICAL_INLINE void StartTransaction()
{
	autortfm_start_transaction();
}

// Manually commit the top transaction nest, popping it from the execution scope.
// Can only be called within an already active parent transaction (EG. this cannot end
// a transaction nest itself).
// Can only be called from the open.
AUTORTFM_DISABLE UE_AUTORTFM_CRITICAL_INLINE void CommitTransaction()
{
	autortfm_commit_transaction();
}

// Optional flags that can be passed as the first template argument to Open().
enum class EOpenFlags
{
	// Default open behavior.
	Default,
	// The AutoRTFM sanitizer will be disabled for the scope of this open.
	NoSanitize,
};

// Executes the given code non-transactionally regardless of whether we are in
// a transaction or not. Returns the value returned by Functor.
// ReturnType must be void or a type that can be safely copied from the open to a closed transaction.
// TAssignFromOpenToClosed must have a specialization for the type that is being returned.
template <EOpenFlags Flags = EOpenFlags::Default, typename FunctorType = void,
	typename ReturnType = decltype(std::declval<FunctorType>()())>
UE_AUTORTFM_CRITICAL_INLINE ReturnType Open(const FunctorType& Functor)
{
#if UE_AUTORTFM
	if (!autortfm_is_closed())
	{
		return Functor();
	}
// Macro filth to either call autortfm_open() or autortfm_open_no_sanitize()
// depending on Flags. Cannot be done as a lambda or inner class method as
// any call indirection will break compiler optimizations.
#define AUTORTFM_INTERNAL_CALL_IN_OPEN(Arg, Function)  \
	do                                                 \
	{                                                  \
		if constexpr (Flags == EOpenFlags::NoSanitize) \
		{                                              \
			autortfm_open_no_sanitize(Function, Arg);  \
		}                                              \
		else                                           \
		{                                              \
			autortfm_open(Function, Arg);              \
		}                                              \
	} while (false)

	if constexpr (std::is_same_v<void, ReturnType>)
	{
		auto Call = [] AUTORTFM_DISABLE(void* ArgPtr)
		{
			auto& Functor = *reinterpret_cast<FunctorType*>(ArgPtr);
			UE_AUTORTFM_CALLSITE_FORCEINLINE Functor();
		};
		AUTORTFM_INTERNAL_CALL_IN_OPEN(const_cast<void*>(reinterpret_cast<const void*>(&Functor)), Call);
	}
	else
	{
		static constexpr EReturnFromOpenMode Mode = ReturnFromOpenModeFor<ReturnType>;
		static_assert(Mode != EReturnFromOpenMode::Unsupported, "function return type is not safe to return from Open()");

		if constexpr (Mode == EReturnFromOpenMode::CopyConstructInClosed || Mode == EReturnFromOpenMode::MoveConstructInClosed)
		{
			alignas(ReturnType) std::byte OpenReturnValueMemory[sizeof(ReturnType)];

			// Call Functor in the open, and in-place construct the return value into OpenReturnValue.
			struct FArgs
			{
				const FunctorType& Functor;
				void* OpenReturnValueMemory;
			};
			FArgs FunctorAndReturnValue{Functor, OpenReturnValueMemory};
			{
				auto Call = [] AUTORTFM_DISABLE(void* ArgPtr)
				{
					FArgs& Args = *reinterpret_cast<FArgs*>(ArgPtr);
					new (Args.OpenReturnValueMemory) ReturnType(Args.Functor());
				};
				AUTORTFM_INTERNAL_CALL_IN_OPEN(&FunctorAndReturnValue, Call);
			}

			// Re-interpret the OpenReturnValueMemory as a ReturnType&.
			ReturnType& OpenReturnValue = *std::launder(reinterpret_cast<ReturnType*>(OpenReturnValueMemory));

			if constexpr (Mode == EReturnFromOpenMode::CopyConstructInClosed)
			{
				// Copy-construct the return value in the closed.
				ReturnType ClosedReturnValue{OpenReturnValue};
				// Destruct OpenReturnValue in the open, if required.
				if constexpr (!std::is_trivially_destructible_v<ReturnType>)
				{
					auto Destruct = [] AUTORTFM_DISABLE(void* ArgPtr)
					{
						reinterpret_cast<ReturnType*>(ArgPtr)->~ReturnType();
					};
					AUTORTFM_INTERNAL_CALL_IN_OPEN(&OpenReturnValue, Destruct);
				}
				return ClosedReturnValue;
			}
			else
			{
				// Move-construct the return value in the closed.
				ReturnType ClosedReturnValue{std::move(OpenReturnValue)};
				// Destruct OpenReturnValue in the open, if required.
				if constexpr (!std::is_trivially_destructible_v<ReturnType>)
				{
					auto Destruct = [] AUTORTFM_DISABLE(void* ArgPtr)
					{
						reinterpret_cast<ReturnType*>(ArgPtr)->~ReturnType();
					};
					AUTORTFM_INTERNAL_CALL_IN_OPEN(&OpenReturnValue, Destruct);
				}
				return ClosedReturnValue;
			}
		}
		else if constexpr (Mode == EReturnFromOpenMode::CustomMethod)
		{
			struct FArgs
			{
				const FunctorType& Functor;
				ReturnType ReturnValue;
			};
			FArgs Args{Functor};
			auto Call = [] AUTORTFM_DISABLE(void* ArgPtr)
			{
				FArgs& Args = *reinterpret_cast<FArgs*>(ArgPtr);
				ReturnType::AutoRTFMAssignFromOpenToClosed(Args.ReturnValue, Args.Functor());
			};
			AUTORTFM_INTERNAL_CALL_IN_OPEN(&Args, Call);
			return Args.ReturnValue;
		}
		else
		{
			static_assert(Mode == EReturnFromOpenMode::Unsupported, "unhandled EReturnFromOpenMode");
		}
	}
#undef AUTORTFM_INTERNAL_CALL_IN_OPEN
#else   // UE_AUTORTFM
	return Functor();
#endif  // UE_AUTORTFM
}

// Always executes the given code transactionally when called from a transaction nest
// (whether we are in open or closed code).
//
// Will crash if called outside of a transaction nest.
//
// If Close() returns an aborting status (see IsStatusAborting()), then
// attempting to use the transaction is undefined behaviour. The caller should
// return to the closed as quickly as possible to avoid the risk of the
// transaction being used in its rolled-back state.
template <typename FunctorType>
[[nodiscard]] UE_AUTORTFM_CRITICAL_INLINE ETransactionStatus Close(AUTORTFM_IMPLICIT_ENABLE const FunctorType& Functor)
{
#if UE_AUTORTFM_ENABLED
	static constexpr autortfm_mode FunctorAutoRTFMMode = CallMode<FunctorType>;
	static_assert(FunctorAutoRTFMMode == autortfm_mode_enable,
		"AutoRTFM mode of functor passed to AutoRTFM::Close() is not autortfm_mode_enable");
#endif
	return static_cast<ETransactionStatus>(autortfm_close(&AutoRTFMFunctorInvoker<FunctorType>,
		AutoRTFMLookupInstrumentedFunctorInvoker<FunctorType>(Functor), const_cast<void*>(static_cast<const void*>(&Functor))));
}

// Similar to Close(), but can be called from outside of a transaction.
// If called from outside a transaction then CloseIfTransactional() will
// call Functor and always return ETransactionStatus::Executing.
template <typename FunctorType>
[[nodiscard]] UE_AUTORTFM_CRITICAL_INLINE ETransactionStatus CloseIfTransactional(AUTORTFM_IMPLICIT_ENABLE FunctorType&& Functor)
{
#if UE_AUTORTFM_ENABLED
	if (IsTransactional())
	{
		return Close(std::forward<FunctorType>(Functor));
	}
	else
	{
		Functor();
		return ETransactionStatus::Executing;
	}
#else
	Functor();
	return ETransactionStatus::Executing;
#endif
}

#if UE_AUTORTFM
// Have some work happen when this transaction commits.
// In a nested transaction, the work is deferred until the outermost nest is committed;
// at that point, the worklist is run in FIFO order.
// If this is called outside a transaction or from an open nest, then the work
// happens immediately.
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void OnCommit(AUTORTFM_IMPLICIT_DISABLE FunctorType&& Work)
{
	if (autortfm_is_closed())
	{
		ForTheRuntime::OnCommitInternal(std::forward<FunctorType>(Work));
	}
	else
	{
		UE_AUTORTFM_CALLSITE_FORCEINLINE Work();
	}
}
#else
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void OnCommit(FunctorType&& Work)
{
	Work();
}
#endif

#if UE_AUTORTFM
// Have some work happen when this transaction aborts (before memory rollback).
// If an abort occurs, the work list is run in LIFO order.
// If this is called outside a transaction or from an open nest then the work is ignored.
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void OnPreAbort(AUTORTFM_IMPLICIT_DISABLE FunctorType&& Work)
{
	if (autortfm_is_closed())
	{
		ForTheRuntime::OnPreAbortInternal(std::forward<FunctorType>(Work));
	}
}
#else
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void OnPreAbort(FunctorType&&)
{
}
#endif

#if UE_AUTORTFM
// Have some work happen when this transaction aborts (after memory rollback).
// If an abort occurs, the work list is run in LIFO order.
// If this is called outside a transaction or from an open nest then the work is ignored.
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void OnAbort(AUTORTFM_IMPLICIT_DISABLE FunctorType&& Work)
{
	if (autortfm_is_closed())
	{
		ForTheRuntime::OnAbortInternal(std::forward<FunctorType>(Work));
	}
}
#else
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void OnAbort(FunctorType&&)
{
}
#endif

#if UE_AUTORTFM
// Enqueues a task to run after the outermost transaction commits or aborts,
// after all other OnCommit() or OnAbort() tasks are run, and after any retries
// of the outermost transaction.
// The worklist is run in FIFO order.
// If this is called outside a transaction or from an open nest, then the work is ignored.
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void OnComplete(AUTORTFM_IMPLICIT_DISABLE FunctorType&& Work)
{
	if (autortfm_is_closed())
	{
		ForTheRuntime::OnCompleteInternal(std::forward<FunctorType>(Work));
	}
}
#else
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void OnComplete(FunctorType&&)
{
}
#endif

#if UE_AUTORTFM
// Enqueues a task to run after the outermost transaction retries due to
// CascadingRetryTransaction(), after all other OnAbort() tasks are run.
// The worklist is run in FIFO order.
// If this is called outside a transaction or from an open nest, then the work is ignored.
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void OnRetry(AUTORTFM_IMPLICIT_DISABLE FunctorType&& Work)
{
	if (autortfm_is_closed())
	{
		ForTheRuntime::OnRetryInternal(std::forward<FunctorType>(Work));
	}
}
#else
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void OnRetry(FunctorType&&)
{
}
#endif

#if UE_AUTORTFM
// Register a handler for transaction commit. Takes a key parameter so that
// the handler can be unregistered (see `PopOnCommitHandler`). This is useful
// for scoped mutations that need an abort handler present unless execution
// reaches the end of the relevant scope.
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void PushOnCommitHandler(const void* Key, AUTORTFM_IMPLICIT_DISABLE FunctorType&& Work)
{
	if (autortfm_is_closed())
	{
		ForTheRuntime::PushOnCommitHandlerInternal(Key, std::forward<FunctorType>(Work));
	}
}
#else
// Register a handler for transaction commit. Takes a key parameter so that
// the handler can be unregistered (see `PopOnCommitHandler`). This is useful
// for scoped mutations that need an abort handler present unless execution
// reaches the end of the relevant scope.
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void PushOnCommitHandler(const void*, AUTORTFM_IMPLICIT_DISABLE FunctorType&&)
{
}
#endif

#if UE_AUTORTFM
// Unregister the most recently pushed handler (via `PushOnCommitHandler`) for the given key.
UE_AUTORTFM_CRITICAL_INLINE void PopOnCommitHandler(const void* Key)
{
	if (autortfm_is_closed())
	{
		ForTheRuntime::PopOnCommitHandlerInternal(Key);
	}
}
#else
// Unregister the most recently pushed handler (via `PushOnCommitHandler`) for the given key.
UE_AUTORTFM_CRITICAL_INLINE void PopOnCommitHandler(const void*) {}
#endif

#if UE_AUTORTFM
// Unregister all pushed handlers (via `PushOnCommitHandler`) for the given key.
UE_AUTORTFM_CRITICAL_INLINE void PopAllOnCommitHandlers(const void* Key)
{
	if (autortfm_is_closed())
	{
		ForTheRuntime::PopAllOnCommitHandlersInternal(Key);
	}
}
#else
// Unregister all pushed handlers (via `PushOnCommitHandler`) for the given key.
UE_AUTORTFM_CRITICAL_INLINE void PopAllOnCommitHandlers(const void*) {}
#endif

#if UE_AUTORTFM
// Register a handler for transaction abort. Takes a key parameter so that
// the handler can be unregistered (see `PopOnAbortHandler`). This is useful
// for scoped mutations that need an abort handler present unless execution
// reaches the end of the relevant scope.
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void PushOnAbortHandler(const void* Key, AUTORTFM_IMPLICIT_DISABLE FunctorType&& Work)
{
	if (autortfm_is_closed())
	{
		ForTheRuntime::PushOnAbortHandlerInternal(Key, std::forward<FunctorType>(Work));
	}
}
#else
// Register a handler for transaction abort. Takes a key parameter so that
// the handler can be unregistered (see `PopOnAbortHandler`). This is useful
// for scoped mutations that need an abort handler present unless execution
// reaches the end of the relevant scope.
template <typename FunctorType>
UE_AUTORTFM_CRITICAL_INLINE void PushOnAbortHandler(const void* Key, FunctorType&&)
{
}
#endif

#if UE_AUTORTFM
// Unregister the most recently pushed handler (via `PushOnAbortHandler`) for the given key.
UE_AUTORTFM_CRITICAL_INLINE void PopOnAbortHandler(const void* Key)
{
	if (autortfm_is_closed())
	{
		ForTheRuntime::PopOnAbortHandlerInternal(Key);
	}
}
#else
// Unregister the most recently pushed handler (via `PushOnAbortHandler`) for the given key.
UE_AUTORTFM_CRITICAL_INLINE void PopOnAbortHandler(const void* Key)
{
	UE_AUTORTFM_UNUSED(Key);
}
#endif

#if UE_AUTORTFM
// Unregister all pushed handlers (via `PushOnAbortHandler`) for the given key.
UE_AUTORTFM_CRITICAL_INLINE void PopAllOnAbortHandlers(const void* Key)
{
	if (autortfm_is_closed())
	{
		ForTheRuntime::PopAllOnAbortHandlersInternal(Key);
	}
}
#else
// Unregister all pushed handlers (via `PushOnAbortHandler`) for the given key.
UE_AUTORTFM_CRITICAL_INLINE void PopAllOnAbortHandlers(const void* Key)
{
	UE_AUTORTFM_UNUSED(Key);
}
#endif

struct FHeapRedirectCallbacks
{
	uint32_t AddressSpace;
	void (*RedirectedLoad)(void* DestPointer, uint64_t Size, uint64_t SourceAddress, bool bWillWriteHint);
	void (*RedirectedLoad8)(void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint);
	void (*RedirectedLoad4)(void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint);
	void (*RedirectedLoad2)(void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint);
	void (*RedirectedLoad1)(void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint);
	void (*RedirectedStore)(uint64_t DestAddress, uint64_t Size, const void* SourcePointer);
	void (*RedirectedStore8)(uint64_t DestAddress, const void* SourcePointer);
	void (*RedirectedStore4)(uint64_t DestAddress, const void* SourcePointer);
	void (*RedirectedStore2)(uint64_t DestAddress, const void* SourcePointer);
	void (*RedirectedStore1)(uint64_t DestAddress, const void* SourcePointer);
};

#if UE_AUTORTFM
UE_AUTORTFM_API void RegisterHeapRedirectCallbacks(FHeapRedirectCallbacks Callbacks);
#else
UE_AUTORTFM_CRITICAL_INLINE void RegisterHeapRedirectCallbacks(FHeapRedirectCallbacks /*Callbacks*/) {}
#endif

// Inform the runtime that we have performed a new object allocation. It's only
// necessary to call this inside of custom malloc implementations. As an
// optimization, you can choose to then only have your malloc return the pointer
// returned by this function. It's guaranteed to be equal to the pointer you
// passed, but it's blessed specially from the compiler's perspective, leading
// to some nice optimizations. This does nothing when called from open code.
UE_AUTORTFM_CRITICAL_INLINE void* DidAllocate(void* Ptr, size_t Size)
{
	return autortfm_did_allocate(Ptr, Size);
}

// Inform the runtime that we have free'd a given memory location.
UE_AUTORTFM_CRITICAL_INLINE void DidFree(void* Ptr)
{
	autortfm_did_free(Ptr);
}

// Informs the runtime that a block of memory is about to be overwritten in the open.
// During a transaction, this allows the runtime to copy the data in preparation for
// a possible abort. Normally, tracking memory overwrites should be automatically
// handled by AutoRTFM, but manual overwrite tracking may be required for third-party
// libraries or outside compilers (such as ISPC).
UE_AUTORTFM_CRITICAL_INLINE void RecordOpenWrite(void* Ptr, size_t Size, EWriteFlags Flags = EWriteFlags::Default)
{
	autortfm_record_open_write_with_flags(Ptr, Size, static_cast<autortfm_write_flags>(Flags | EWriteFlags::NoSanitize));
}

// Informs the runtime that a block of memory is about to be overwritten in the open.
// During a transaction, this allows the runtime to copy the data in preparation for
// a possible abort. Normally, tracking memory overwrites should be automatically
// handled by AutoRTFM, but manual overwrite tracking may be required for third-party
// libraries or outside compilers (such as ISPC).
template <typename Type>
UE_AUTORTFM_CRITICAL_INLINE void RecordOpenWrite(Type* Ptr, EWriteFlags Flags = EWriteFlags::Default)
{
	autortfm_record_open_write_with_flags(Ptr, sizeof(Type), static_cast<autortfm_write_flags>(Flags | EWriteFlags::NoSanitize));
}

// Performs a transactional assignment in the open, recording the existing value
// of Ref before changing it to NewValue. Can be called from the open or closed.
#if UE_AUTORTFM_ENABLED
template <typename RefType, typename ValueType>
AUTORTFM_OPEN_NO_SANITIZE void Assign(RefType& Ref, ValueType&& NewValue, EWriteFlags Flags = EWriteFlags::Default)
{
	autortfm_record_open_write_with_flags(&Ref, sizeof(Ref), static_cast<autortfm_write_flags>(Flags));
	{
		AUTORTFM_SANITIZER_DISABLE_SCOPE();
		Ref = std::forward<ValueType>(NewValue);
	}
}
#else
template <typename RefType, typename ValueType>
UE_AUTORTFM_CRITICAL_INLINE void Assign(RefType& Ref, ValueType&& NewValue, EWriteFlags Flags = EWriteFlags::Default)
{
	UE_AUTORTFM_UNUSED(Flags);
	Ref = std::forward<ValueType>(NewValue);
}
#endif

// A RAII helper to call AutoRTFM::Assign(Variable, NewValue) on construction
// and again on destruction with the old value of Variable.
template <typename DataType>
struct AUTORTFM_DISABLE_NO_SANITIZE TScopedAssign
{
	TScopedAssign(DataType& Variable, DataType NewValue) : Variable{Variable}, OldValue{Variable}
	{
		Assign(Variable, NewValue);
	}
	~TScopedAssign()
	{
		Assign(Variable, OldValue);
	}

private:
	TScopedAssign(DataType& Variable) = delete;
	const TScopedAssign& operator=(DataType& Variable) = delete;
	DataType& Variable;
	DataType OldValue;
};

// Report that a unreachable codepath is being hit. Used to manually ban certain codepaths
// from being transactionally safe.
#if UE_AUTORTFM_ENABLED
[[noreturn]] UE_AUTORTFM_CRITICAL_INLINE_ALWAYS void Unreachable(const char* Message = nullptr)
{
	autortfm_unreachable(Message);
}
#else
UE_AUTORTFM_CRITICAL_INLINE void Unreachable(const char* Message = nullptr)
{
	UE_AUTORTFM_UNUSED(Message);
}
#endif

// If we are running within a transaction, call `AutoRTFM::Unreachable`.
UE_AUTORTFM_CRITICAL_INLINE_ALWAYS void UnreachableIfTransactional(const char* Message = nullptr)
{
	if (AutoRTFM::IsTransactional())
	{
		AutoRTFM::Unreachable(Message);
	}
}

// If we are running within a closed transaction, call `AutoRTFM::Unreachable`.
UE_AUTORTFM_CRITICAL_INLINE_ALWAYS void UnreachableIfClosed(const char* Message = nullptr)
{
	if (AutoRTFM::IsClosed())
	{
		AutoRTFM::Unreachable(Message);
	}
}

// A collection of power-user functions that are reserved for use by the AutoRTFM runtime only.
namespace ForTheRuntime
{
// An enum to represent the various ways we want to enable/disable the AutoRTFM runtime.
// This enum has effective groups of functionality such that if a higher priority group
// has enabled or disabled the runtime, a lower priority group cannot then override that.
//
// We have from higher to lower priority:
// - Forced Enabled/Disabled - used by CVars when force enabling/disabling AutoRTFM.
// - Override Enabled/Disabled - override any setting of enabled/disabled as was set by a CVar.
// - Enabled/Disabled - used by CVars when enabling/disabling AutoRTFM.
// - Default Enabled/Disabled - whether we should be enabled or disabled by default (used for different backend executables).
//
// For example the following would be valid:
// - At compile time the state is compiled in as default disabled.
// - The CVar is set to enabled, so we switch the state to enabled.
// - At runtime we detect a mode where we want AutoRTFM and try to switch the default to enabled, but the CVar already enabled it so
// this is ignored.
// - Then for a given codepath we override AutoRTFM to disabled so we switch the state to disabled.
enum EAutoRTFMEnabledState
{
	// Disable AutoRTFM.
	AutoRTFM_Disabled = 0,

	// Enable AutoRTFM.
	AutoRTFM_Enabled,

	// Force disable AutoRTFM.
	AutoRTFM_ForcedDisabled,

	// Force enable AutoRTFM.
	AutoRTFM_ForcedEnabled,

	// Whether our default is to be disabled.
	AutoRTFM_DisabledByDefault,

	// Whether our default is to be enabled.
	AutoRTFM_EnabledByDefault,

	// Whether we've overridden and AutoRTFM is disabled.
	AutoRTFM_OverriddenDisabled,

	// Whether we've overridden and AutoRTFM is enabled.
	AutoRTFM_OverriddenEnabled,
};

// An enum to represent whether we should abort and retry transactions (for testing purposes).
enum EAutoRTFMRetryTransactionState
{
	// Do not abort and retry transactions (the default).
	NoRetry = 0,

	// Abort and retry non-nested transactions (EG. only abort the parent transactional nest).
	RetryNonNested,

	// Abort and retry nested-transactions too. Will be slower as each nested-transaction will
	// be aborted and retried at least *twice* (once when the non-nested transaction runs the
	// first time, and a second time when the non-nested transaction is doing its retry after
	// aborting).
	RetryNestedToo,
};

enum EAutoRTFMInternalAbortActionState
{
	// Crash the process if we hit an internal AutoRTFM abort.
	Crash = 0,

	// Just do a normal transaction abort and let the runtime recover (used to test aborting codepaths).
	Abort,
};

// Set whether the AutoRTFM runtime is enabled or disabled. Returns true when the state was changed
// successfully.
UE_AUTORTFM_API bool SetAutoRTFMRuntime(EAutoRTFMEnabledState State);

UE_AUTORTFM_API bool IsAutoRTFMRuntimeEnabledInternal();

// Query whether the AutoRTFM runtime is enabled.
UE_AUTORTFM_CRITICAL_INLINE bool IsAutoRTFMRuntimeEnabled()
{
	// If we are already in the closed nest of a transaction, we must have our runtime enabled!
	if (AutoRTFM::IsClosed())
	{
		return true;
	}

	return IsAutoRTFMRuntimeEnabledInternal();
}

// Set the percentage [0..100] chance that a call to `CoinTossDisable` will end up disabling AutoRTFM.
// 100% means never disable via coin-toss, 0% means always disable. So passing `0.1` means disable
// all but 1/1000's calls via `CoinTossDisable`.
UE_AUTORTFM_API void SetAutoRTFMEnabledProbability(float Chance);

// Get the enabled probability set via `SetAutoRTFMEnabledProbability`.
UE_AUTORTFM_API float GetAutoRTFMEnabledProbability();

// Call to randomly disable AutoRTFM with a probability set with `SetAutoRTFMEnabledProbability`.
// Returns true if AutoRTFM was disabled by this call.
UE_AUTORTFM_API bool CoinTossDisable();

UE_AUTORTFM_API void SetInternalAbortAction(EAutoRTFMInternalAbortActionState State);

UE_AUTORTFM_API EAutoRTFMInternalAbortActionState GetInternalAbortAction();

UE_AUTORTFM_API bool GetEnsureOnInternalAbort();
UE_AUTORTFM_API void SetEnsureOnInternalAbort(bool bEnabled);

UE_AUTORTFM_API autortfm_log_severity GetAutoRTFMHazardLogSeverity();
UE_AUTORTFM_API void SetAutoRTFMHazardLogSeverity(autortfm_log_severity Severity);

// Set whether we should trigger an ensure on an abort-by-language.
[[deprecated("Use `SetEnsureOnInternalAbort` instead!")]] inline void SetEnsureOnAbortByLanguage(bool bEnabled)
{
	SetEnsureOnInternalAbort(bEnabled);
}

// Returns whether the runtime will trigger an ensure on an abort-by-language, or not.
[[deprecated("Use `GetEnsureOnInternalAbort` instead!")]] inline bool IsEnsureOnAbortByLanguageEnabled()
{
	return GetEnsureOnInternalAbort();
}

// Returns whether we want to assert or ensure on a Language Error
[[deprecated("Use `GetInternalAbortAction` instead!")]] inline bool IsAutoRTFMAssertOnError()
{
	return EAutoRTFMInternalAbortActionState::Crash == GetInternalAbortAction();
}

// Set whether we should retry transactions.
UE_AUTORTFM_API void SetRetryTransaction(EAutoRTFMRetryTransactionState State);

// Returns whether we should retry transactions.
UE_AUTORTFM_API EAutoRTFMRetryTransactionState GetRetryTransaction();

// Returns true if we should retry non-nested transactions.
UE_AUTORTFM_API bool ShouldRetryNonNestedTransactions();

// Returns true if we should also retry nested transactions.
UE_AUTORTFM_API bool ShouldRetryNestedTransactionsToo();

// Reserved for future.
UE_AUTORTFM_CRITICAL_INLINE void RecordOpenRead(void const*, size_t) {}

// Reserved for future.
template <typename Type>
UE_AUTORTFM_CRITICAL_INLINE void RecordOpenRead(Type*)
{
}

}  // namespace ForTheRuntime

}  // namespace AutoRTFM

// Macro-based variants so we completely compile away when not in use, even in debug builds
#if UE_AUTORTFM

namespace AutoRTFM::Private
{
struct FOpenHelper
{
	template <typename FunctorType>
	UE_AUTORTFM_FORCEINLINE void operator+(FunctorType&& F)
	{
		AutoRTFM::Open(std::forward<FunctorType>(F));
	}
};
struct FOpenNoSanitizeHelper
{
	template <typename FunctorType>
	UE_AUTORTFM_FORCEINLINE void operator+(FunctorType&& F)
	{
		AutoRTFM::Open<EOpenFlags::NoSanitize>(std::forward<FunctorType>(F));
	}
};
struct FCloseHelper
{
	template <typename FunctorType>
	[[nodiscard]] EContextStatus operator+(FunctorType F)
	{
		return AutoRTFM::Close(std::move(F));
	}
};
struct FOnPreAbortHelper
{
	template <typename FunctorType>
	UE_AUTORTFM_FORCEINLINE void operator+(FunctorType&& F)
	{
		AutoRTFM::OnPreAbort(std::forward<FunctorType>(F));
	}
};
struct FOnAbortHelper
{
	template <typename FunctorType>
	UE_AUTORTFM_FORCEINLINE void operator+(FunctorType&& F)
	{
		AutoRTFM::OnAbort(std::forward<FunctorType>(F));
	}
};
struct FOnCommitHelper
{
	template <typename FunctorType>
	UE_AUTORTFM_FORCEINLINE void operator+(FunctorType&& F)
	{
		AutoRTFM::OnCommit(std::forward<FunctorType>(F));
	}
};
struct FTransactHelper
{
	template <typename FunctorType>
	UE_AUTORTFM_FORCEINLINE void operator+(FunctorType&& F)
	{
		AutoRTFM::Transact(std::forward<FunctorType>(F));
	}
};
namespace /* must have internal linkage */
{
struct FThreadLocalHelper
{
	template <typename Type, int Unique>
	UE_AUTORTFM_ALWAYS_OPEN static Type& Get()
	{
		thread_local Type Data;
		return Data;
	}
};
}
}  // namespace AutoRTFM::Private

#define UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR_IMPL(Type, Name) \
	Type& Name = ::AutoRTFM::Private::FThreadLocalHelper::Get<Type, __COUNTER__>()

#define UE_AUTORTFM_OPEN_IMPL ::AutoRTFM::Private::FOpenHelper{} + [&] AUTORTFM_DISABLE()
#define UE_AUTORTFM_OPEN_NO_SANITIZE_IMPL ::AutoRTFM::Private::FOpenNoSanitizeHelper{} + [&] AUTORTFM_DISABLE_NO_SANITIZE()
#define UE_AUTORTFM_CLOSE_IMPL ::AutoRTFM::Private::FCloseHelper{} + [&] AUTORTFM_ENABLE()
#define UE_AUTORTFM_ONPREABORT_IMPL(...) ::AutoRTFM::Private::FOnPreAbortHelper{} + [__VA_ARGS__] AUTORTFM_DISABLE() mutable
#define UE_AUTORTFM_ONABORT_IMPL(...) ::AutoRTFM::Private::FOnAbortHelper{} + [__VA_ARGS__] AUTORTFM_DISABLE() mutable
#define UE_AUTORTFM_ONCOMMIT_IMPL(...) ::AutoRTFM::Private::FOnCommitHelper{} + [__VA_ARGS__] AUTORTFM_DISABLE() mutable
#define UE_AUTORTFM_TRANSACT_IMPL ::AutoRTFM::Private::FTransactHelper{} + [&] AUTORTFM_ENABLE()
#else

// Do nothing, these should be followed by blocks that should be either executed or not executed
#define UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR_IMPL(Type, Name) thread_local Type Name
#define UE_AUTORTFM_OPEN_IMPL
#define UE_AUTORTFM_OPEN_NO_SANITIZE_IMPL
#define UE_AUTORTFM_CLOSE_IMPL
#define UE_AUTORTFM_ONPREABORT_IMPL(...) (void)[__VA_ARGS__]() mutable
#define UE_AUTORTFM_ONABORT_IMPL(...) (void)[__VA_ARGS__]() mutable
#define UE_AUTORTFM_ONCOMMIT_IMPL(...)
#define UE_AUTORTFM_TRANSACT_IMPL
#endif

// Declares an AutoRTFM-aware thread local variable. `thread_local` variables are not yet natively supported (#jira SOL-7684)
// This macro must be used inside a function, and will not work properly at global scope. Calls should be written like this:
//     UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(FString, MyThreadLocalString);
//     MyThreadLocalString = TEXT("Hello");
#define UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(Type, Name) UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR_IMPL(Type, Name)

// Runs a block of code in the open, non-transactionally. Anything performed in the open will not be undone if a transaction fails.
// Calls should be written like this: UE_AUTORTFM_OPEN { ... code ... };
#define UE_AUTORTFM_OPEN UE_AUTORTFM_OPEN_IMPL

// Deprecated: Use UE_AUTORTFM_OPEN_NO_SANITIZE
#define UE_AUTORTFM_OPEN_NO_VALIDATION UE_AUTORTFM_OPEN_NO_SANITIZE_IMPL

// Similar to UE_AUTORTFM_OPEN, but the AutoRTFM sanitizer will be disabled
// for the scope of the call.
#define UE_AUTORTFM_OPEN_NO_SANITIZE UE_AUTORTFM_OPEN_NO_SANITIZE_IMPL

// Runs a block of code in the closed, transactionally. Anything performed in the closed will be undone if a transaction fails.
// Calls should be written like this: UE_AUTORTFM_CLOSE { ... code ... };
#define UE_AUTORTFM_CLOSE UE_AUTORTFM_CLOSE_IMPL

// Runs a block of code if a transaction aborts (before memory rollback).
// In non-transactional code paths the block of code will not be executed at all.
// The macro arguments are the capture specification for the lambda.
// Calls should be written like this: UE_AUTORTFM_ONPREABORT(=) { ... code ... };
#define UE_AUTORTFM_ONPREABORT(...) UE_AUTORTFM_ONPREABORT_IMPL(__VA_ARGS__)

// Runs a block of code if a transaction aborts (after memory rollback).
// In non-transactional code paths the block of code will not be executed at all.
// The macro arguments are the capture specification for the lambda.
// Calls should be written like this: UE_AUTORTFM_ONABORT(=) { ... code ... };
#define UE_AUTORTFM_ONABORT(...) UE_AUTORTFM_ONABORT_IMPL(__VA_ARGS__)

// Runs a block of code if a transaction commits successfully.
// In non-transactional code paths the block of code will be executed immediately.
// The macro arguments are the capture specification for the lambda.
// Calls should be written like this: UE_AUTORTFM_ONCOMMIT(=) { ... code ... };
#define UE_AUTORTFM_ONCOMMIT(...) UE_AUTORTFM_ONCOMMIT_IMPL(__VA_ARGS__)

// Runs a block of code in the closed, transactionally, within a new transaction.
// Calls should be written like this: UE_AUTORTFM_TRANSACT { ... code ... };
#define UE_AUTORTFM_TRANSACT UE_AUTORTFM_TRANSACT_IMPL

#if UE_AUTORTFM
#define UE_AUTORTFM_REGISTER_OPEN_TO_CLOSED_FUNCTIONS(...)                                                      \
	static const ::AutoRTFM::ForTheRuntime::TAutoRegisterOpenToClosedFunctions<__VA_ARGS__> UE_AUTORTFM_CONCAT( \
		AutoRTFMFunctionRegistration, __COUNTER__)
#else
#define UE_AUTORTFM_REGISTER_OPEN_TO_CLOSED_FUNCTIONS(...)
#endif

#endif  // __cplusplus
