// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/Defines.h"
#include "Constants.h"
#include "Defines.h"

#include <cstdarg>  // for va_list
#include <cstddef>  // for size_t

// Define UE_AUTORTFM_API (DLL export / import attributes)
#if defined(__cplusplus) && defined(__UNREAL__) \
	&& !(defined(UE_AUTORTFM_DO_NOT_INCLUDE_PLATFORM_H) && UE_AUTORTFM_DO_NOT_INCLUDE_PLATFORM_H)
// Include HAL/Platform.h for DLLIMPORT / DLLEXPORT definitions, which
// UBT can use as a definition for AUTORTFM_API.
#include <HAL/Platform.h>
#define UE_AUTORTFM_API AUTORTFM_API
#else
#ifndef UE_AUTORTFM_API
#define UE_AUTORTFM_API
#endif
#endif

#ifdef __cplusplus
extern "C"
{
#endif

	// The C API exists for a few reasons:
	//
	// - It makes linking easy. AutoRTFM has to deal with a weird kind of linking
	//   where the compiler directly emits calls to functions with a given name.
	//   It's easiest to do that in llvm if the functions have C linkage and C ABI.
	// - It makes testing easy. Even seemingly simple C++ code introduces pitfalls
	//   for AutoRTFM. So very focused tests work best when written in C.
	// - It makes compiler optimizations much easier as there is no mangling to
	// 	 consider when looking for functions in the runtime we can optimize.
	//
	// We use snake_case for C API surface area to make it easy to distinguish.
	//
	// The C API should not be used directly - it is here purely as an
	// implementation detail.

	// This must match values in AutoRTFM::ETransactionStatus / AutoRTFM::ETransactionResult.
	typedef enum
	{
		autortfm_transaction_executing = 0,
		autortfm_transaction_committed,
		autortfm_transaction_aborted_by_request,
		autortfm_transaction_aborted_by_language,
		autortfm_transaction_rejected_during_unwind,
		autortfm_transaction_rejected_during_commit,
		autortfm_transaction_rejected_during_abort,
		autortfm_transaction_rejected_during_retry,
		autortfm_transaction_rejected_during_complete,
		autortfm_transaction_aborted_by_cascading_abort,
		autortfm_transaction_aborted_by_cascading_retry,
	} autortfm_transaction_status;

	// AutoRTFM logging severity.
	typedef enum
	{
		autortfm_log_verbose = 0,
		autortfm_log_info,
		autortfm_log_warn,
		autortfm_log_error,
		autortfm_log_fatal,
	} autortfm_log_severity;

	// Flags that can be passed to autortfm_record_open_write_with_flags().
	typedef enum
	{
		// No flags applied to the write.
		autortfm_write_default = 0x0,
		// The write should not be considered by the AutoRTFM sanitizer.
		autortfm_write_no_sanitize = 0x1,
		// The autortfm_extern_api::RollbackWrite will be called to perform
		// the rollback for this write if the transaction is aborted.
		autortfm_write_custom_rollback = 0x2,
		// Custom flag bits which can be used by the custom abort callbacks.
		// Requires autortfm_write_custom_rollback for usage.
		autortfm_write_custom_flag_0 = 0x4,
		autortfm_write_custom_flag_1 = 0x8,
	} autortfm_write_flags;

	// An opaque unique identifier for a transaction.
	typedef uint64_t autortfm_transaction_id;

	// A single frame of a callstack capture.
	typedef unsigned long long autortfm_callstack_frame;

	// Function pointers used by AutoRTFM for heap allocations, etc.
	typedef struct
	{
		// The function used to allocate memory from the heap. Must not be null.
		// This API should never return null; if the allocator cannot satisfy the request, it must abort the program.
		void* (*Allocate)(size_t Size, size_t Alignment);

		// The function used to reallocate memory from the heap. Must not be null.
		// This API should never return null; if the allocator cannot satisfy the request, it must abort the program.
		void* (*Reallocate)(void* Pointer, size_t Size, size_t Alignment, size_t PreviousSize);

		// The function used to allocate zeroed memory from the heap. Must not be null.
		// This API should never return null; if the allocator cannot satisfy the request, it must abort the program.
		void* (*AllocateZeroed)(size_t Size, size_t Alignment);

		// The function used to free memory allocated by Allocate(), Reallocate() or AllocateZeroed().
		// Must not be null.
		void (*Free)(void* Pointer, size_t AllocationSize);

		// Function used to log messages using a printf-style format string and va_list arguments.
		// Strings use UTF-8 encoding.
		// Must not be null.
		void (*Log)(
			const char* File, int Line, void* ProgramCounter, autortfm_log_severity Severity, const char* Format, va_list Args);

		// Function used to log messages with a callstack using a printf-style format string and va_list arguments.
		// Strings use UTF-8 encoding.
		// Must not be null.
		void (*LogWithCallstack)(void* ProgramCounter, autortfm_log_severity Severity, const char* Format, va_list Args);

		// Function used to report an ensure failure using a printf-style format string and va_list arguments.
		// Strings use UTF-8 encoding.
		// Must not be null.
		void (*EnsureFailure)(
			const char* File, int Line, void* ProgramCounter, const char* Condition, const char* Format, va_list Args);

		// Function used to query whether a log severity is active.
		// Must not be null.
		bool (*IsLogActive)(autortfm_log_severity Severity);

		// Performs a capture of the currently executing callstack, writing at most MaxFrames frame
		// addresses to StackOut, starting with with the most deeply nested frame.
		// Returns the number of frame addresses actually written.
		// Must not be null.
		size_t (*CaptureCallstack)(size_t MaxFrames, autortfm_callstack_frame* StackOut);

		// Prints a callstack captured with CaptureCallstack to the log with the
		// given severity.
		// Must not be null.
		void (*LogCallstack)(autortfm_log_severity Severity, size_t Count, autortfm_callstack_frame const* Stack);

		// Optional callback to be informed when the value returned by
		// ForTheRuntime::IsAutoRTFMRuntimeEnabled() changes.
		// Can be null.
		void (*OnRuntimeEnabledChanged)();

		// Optional callback to be informed when the value returned by
		// ForTheRuntime::GetRetryTransaction() changes.
		// Can be null.
		void (*OnRetryTransactionsChanged)();

#if AUTORTFM_SANITIZER
		// Optional callback to be informed when the value returned by
		// ForTheRuntime::GetSanitizerMode() changes.
		// Can be null.
		void (*OnSanitizerModeChanged)();

		// Optional callback to be informed when the value returned by
		// ForTheRuntime::SanitizerRecordClosedWriteCallstacks() changes.
		// Can be null.
		void (*OnSanitizerRecordClosedWriteCallstacksChanged)();
#endif  // AUTORTFM_SANITIZER

		// Custom abort handler for writes that that were added with the
		// autortfm_write_custom_rollback flag. It is the responsibility of this
		// function to perform the rollback (copy of Size bytes from Value to
		// Address).
		// RollbackWrite may contain multiple writes condensed into a single
		// contigious block of memory, and the writes may be split into multiple
		// calls with no defined size or alignment granularity.
		// Calls are ordered starting with the most recently recorded writes.
		// Address / Value pointers are ordered starting with the least recently
		// recorded writes.
		// Split writes are guaranteed to be "completed" with the next immediate
		// call(s).
		void (*RollbackWrite)(void* Address, const void* Value, size_t Size, autortfm_write_flags Flags);
	} autortfm_extern_api;

#if UE_AUTORTFM_ENABLED
	// Initialize the AutoRTFM library.
	// Must only be called once for the lifetime of the application.
	// Parameters:
	//   ExternAPI - Function pointers used by AutoRTFM for heap allocations, etc.
	//               Must be non-null.
	UE_AUTORTFM_API void autortfm_initialize(const autortfm_extern_api* ExternAPI) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_initialize(const autortfm_extern_api* ExternAPI) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(ExternAPI);
}
#endif

#if UE_AUTORTFM_ENABLED
	// Shutdown the AutoRTFM library.
	// Must only be called once for the lifetime of the application, after autortfm_initialize().
	UE_AUTORTFM_API void autortfm_shutdown() AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_shutdown() AUTORTFM_NOEXCEPT {}
#endif

#if UE_AUTORTFM_ENABLED
	// Note: There is no implementation of this function.
	// The AutoRTFM compiler will replace all calls to this function with a constant boolean value.
	UE_AUTORTFM_API bool autortfm_is_closed(void) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE bool autortfm_is_closed(void) AUTORTFM_NOEXCEPT
{
	return false;
}
#endif

// Debug utility: place a call to this function in a function body to have the
// AutoRTFM compiler dump that function's IR at each pipeline stage (pre/post
// EarlyPass, pre/post InstrumentationPass). The call is stripped after the
// final dump. Has no effect when AutoRTFM is disabled.
#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API void autortfm_debug_dump_ir(void) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_debug_dump_ir(void) AUTORTFM_NOEXCEPT {}
#endif

#if UE_AUTORTFM_ENABLED
	// Returns the current AutoRTFM context status.
	// This function is handled specially in the compiler and will be constant
	// folded as autortfm_status_on_track in closed code, or preserved as a
	// function call in open code.
	UE_AUTORTFM_API autortfm_context_status autortfm_get_context_status(void) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE autortfm_context_status autortfm_get_context_status(void) AUTORTFM_NOEXCEPT
{
	return autortfm_status_idle;
}
#endif

#if UE_AUTORTFM_ENABLED
	// Returns true if the AutoRTFM context status is equal to Status.
	// This is equivallent to (autortfm_get_context_status() == Status) and exists
	// as a constant-folding optimization for non-optimized builds.
	// This function is handled specially in the compiler and will be constant
	// folded if possible.
	UE_AUTORTFM_API bool autortfm_is_context_status(autortfm_context_status Status) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE bool autortfm_is_context_status(autortfm_context_status Status) AUTORTFM_NOEXCEPT
{
	return Status == autortfm_status_idle;
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API autortfm_transaction_status autortfm_transact(
		void (*UninstrumentedWork)(void*), void (*InstrumentedWork)(void*), void* Arg) AUTORTFM_EXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE autortfm_transaction_status autortfm_transact(
	void (*UninstrumentedWork)(void*), void (*InstrumentedWork)(void*), void* Arg) AUTORTFM_EXCEPT
{
	UE_AUTORTFM_UNUSED(InstrumentedWork);
	UninstrumentedWork(Arg);
	return autortfm_transaction_committed;
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API autortfm_transaction_status autortfm_transact_then_open(
		void (*UninstrumentedWork)(void*), void* Arg) AUTORTFM_EXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE autortfm_transaction_status autortfm_transact_then_open(
	void (*UninstrumentedWork)(void*), void* Arg) AUTORTFM_EXCEPT
{
	UninstrumentedWork(Arg);
	return autortfm_transaction_committed;
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API UE_AUTORTFM_FORCENOINLINE void autortfm_abort_transaction() AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_abort_transaction() AUTORTFM_NOEXCEPT {}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API UE_AUTORTFM_FORCENOINLINE void autortfm_cascading_abort_transaction() AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_cascading_abort_transaction() AUTORTFM_NOEXCEPT {}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API UE_AUTORTFM_FORCENOINLINE void autortfm_cascading_retry_transaction() AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_cascading_retry_transaction() AUTORTFM_NOEXCEPT {}
#endif

#if UE_AUTORTFM_ENABLED
	AUTORTFM_DISABLE UE_AUTORTFM_API void autortfm_start_transaction() AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_start_transaction() AUTORTFM_NOEXCEPT {}
#endif

#if UE_AUTORTFM_ENABLED
	AUTORTFM_DISABLE UE_AUTORTFM_API void autortfm_commit_transaction() AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_commit_transaction() AUTORTFM_NOEXCEPT {}
#endif

#if UE_AUTORTFM_ENABLED
	AUTORTFM_OPEN UE_AUTORTFM_API autortfm_transaction_id autortfm_current_transaction_id() AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE autortfm_transaction_id autortfm_current_transaction_id() AUTORTFM_NOEXCEPT
{
	return 0;
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_FORCENOINLINE UE_AUTORTFM_API void autortfm_open(void (*work)(void* arg), void* arg);
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_open(void (*work)(void* arg), void* arg)
{
	work(arg);
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_FORCENOINLINE UE_AUTORTFM_API void autortfm_open_no_sanitize(void (*work)(void* arg), void* arg);
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_open_no_sanitize(void (*work)(void* arg), void* arg)
{
	work(arg);
}
#endif

#if UE_AUTORTFM_ENABLED
	[[nodiscard]] UE_AUTORTFM_API autortfm_transaction_status autortfm_close(
		void (*UninstrumentedWork)(void*), void (*InstrumentedWork)(void*), void* Arg);
#else
AUTORTFM_DISABLE_UNREACHABLE_CODE_WARNINGS
[[nodiscard]] UE_AUTORTFM_CRITICAL_INLINE autortfm_transaction_status autortfm_close(
	void (*UninstrumentedWork)(void*), void (*InstrumentedWork)(void*), void* Arg)
{
	UE_AUTORTFM_UNUSED(UninstrumentedWork);
	UE_AUTORTFM_UNUSED(InstrumentedWork);
	UE_AUTORTFM_UNUSED(Arg);
#if defined(__clang__)
	__builtin_trap();
#endif
	return autortfm_transaction_aborted_by_request;
}
AUTORTFM_RESTORE_UNREACHABLE_CODE_WARNINGS
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API void autortfm_record_open_write(void* Ptr, size_t Size) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_record_open_write(void* Ptr, size_t Size) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(Ptr);
	UE_AUTORTFM_UNUSED(Size);
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API void autortfm_record_open_write_with_flags(void* Ptr, size_t Size, autortfm_write_flags Flags) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_record_open_write_with_flags(
	void* Ptr, size_t Size, autortfm_write_flags Flags) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(Ptr);
	UE_AUTORTFM_UNUSED(Size);
	UE_AUTORTFM_UNUSED(Flags);
}
#endif

	// autortfm_open_to_closed_mapping maps an open function to its closed variant.
	struct autortfm_open_to_closed_mapping
	{
		void* Open;
		void* Closed;
	};

	// autortfm_open_to_closed_table holds a pointer to a null-terminated list of
	// autortfm_open_to_closed_mapping, and an intrusive linked-list pointer to the
	// previous and next registered autortfm_open_to_closed_table.
	struct autortfm_open_to_closed_table
	{
		// Null-terminated open function to closed function mapping table.
		const struct autortfm_open_to_closed_mapping* Mappings;
		// An intrusive linked-list pointer to the previous autortfm_open_to_closed_table.
		// Used by autortfm_register_open_to_closed_functions().
		struct autortfm_open_to_closed_table* Prev;
		// An intrusive linked-list pointer to the next autortfm_open_to_closed_table.
		// Used by autortfm_register_open_to_closed_functions().
		struct autortfm_open_to_closed_table* Next;
	};

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API void autortfm_register_open_to_closed_functions(struct autortfm_open_to_closed_table* Table) AUTORTFM_NOEXCEPT;
	UE_AUTORTFM_API void autortfm_unregister_open_to_closed_functions(struct autortfm_open_to_closed_table* Table) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_register_open_to_closed_functions(autortfm_open_to_closed_table* Table) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(Table);
}
UE_AUTORTFM_CRITICAL_INLINE void autortfm_unregister_open_to_closed_functions(autortfm_open_to_closed_table* Table) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(Table);
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API bool autortfm_is_on_current_transaction_stack(void* Ptr) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE bool autortfm_is_on_current_transaction_stack(void* Ptr) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(Ptr);
	return false;
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API void autortfm_on_commit(void (*work)(void* arg), void* arg);
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_on_commit(void (*work)(void* arg), void* arg)
{
	work(arg);
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API void autortfm_on_pre_abort(void (*work)(void* arg), void* arg) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_on_pre_abort(void (*work)(void* arg), void* arg) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(work);
	UE_AUTORTFM_UNUSED(arg);
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API void autortfm_on_abort(void (*work)(void* arg), void* arg) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_on_abort(void (*work)(void* arg), void* arg) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(work);
	UE_AUTORTFM_UNUSED(arg);
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API void autortfm_push_on_abort_handler(const void* key, void (*work)(void* arg), void* arg) AUTORTFM_NOEXCEPT;
	UE_AUTORTFM_API void autortfm_pop_on_abort_handler(const void* key) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_push_on_abort_handler(
	const void* key, void (*work)(void* arg), void* arg) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(key);
	UE_AUTORTFM_UNUSED(work);
	UE_AUTORTFM_UNUSED(arg);
}

UE_AUTORTFM_CRITICAL_INLINE void autortfm_pop_on_abort_handler(const void* key) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(key);
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API void* autortfm_did_allocate(void* ptr, size_t size) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void* autortfm_did_allocate(void* ptr, size_t size) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(size);
	return ptr;
}
#endif

#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API void autortfm_did_free(void* ptr) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_did_free(void* ptr) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(ptr);
}
#endif

// If running with AutoRTFM enabled, then perform an ABI check between the
// AutoRTFM compiler and the AutoRTFM runtime, to ensure that memory is being
// laid out in an identical manner between the AutoRTFM runtime and the AutoRTFM
// compiler pass. Should not be called manually by the user, a call to this will
// be injected by the compiler into a global constructor in the AutoRTFM compiled
// code.
#if UE_AUTORTFM_ENABLED
	UE_AUTORTFM_API void autortfm_check_abi(void* ptr, size_t size) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_check_abi(void* ptr, size_t size) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(ptr);
	UE_AUTORTFM_UNUSED(size);
}
#endif

#if UE_AUTORTFM_ENABLED
	// Called when execution unexpectedly reaches a code path that was considered unreachable.
	// Either aborts execution of the program or aborts the current transaction, depending on the
	// current 'InternalAbortAction' state.
	[[noreturn]] AUTORTFM_OPEN UE_AUTORTFM_API void autortfm_unreachable(const char* Message) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void autortfm_unreachable(const char* Message) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(Message);
}
#endif

#if UE_AUTORTFM_ENABLED
	// Given an open function pointer, returns the equivalent closed function pointer.
	extern "C" UE_AUTORTFM_FORCENOINLINE UE_AUTORTFM_API void* autortfm_lookup_function(
		void* OriginalFunction, const char* Where) AUTORTFM_NOEXCEPT;
#else
UE_AUTORTFM_CRITICAL_INLINE void* autortfm_lookup_function(void* OriginalFunction, const char* Where) AUTORTFM_NOEXCEPT
{
	UE_AUTORTFM_UNUSED(OriginalFunction);
	UE_AUTORTFM_UNUSED(Where);
	return nullptr;
}
#endif

#ifdef __cplusplus
}  // extern "C"
#endif
