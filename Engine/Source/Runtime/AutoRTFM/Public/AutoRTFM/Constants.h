// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// WARNING: Any change in these constants will require a re-patch and re-build of LLVM!

#ifdef __cplusplus
#include <cstdint>

extern "C"
{
#endif
	// An enumerator of AutoRTFM context status modes.
	// This must match AutoRTFM::EContextStatus.
	typedef enum
	{
		autortfm_status_idle = 0,
		autortfm_status_on_track,
		autortfm_status_unwinding,
		autortfm_status_committing,
		autortfm_status_aborting,
		autortfm_status_retrying,
		autortfm_status_completing,
		autortfm_status_in_static_local_initializer,
	} autortfm_context_status;

	// An enumerator of AutoRTFM sanitizer modes.
	// The AutoRTFM sanitizer is used to detect modification by open-code to memory that was written by
	// a transaction. In this situation, aborting the transaction can corrupt memory as the undo will
	// overwrite the writes made in the open-code.
	typedef enum
	{
		// Disable AutoRTFM sanitizer.
		autortfm_sanitizer_disabled,

		// Enable AutoRTFM sanitizer.
		// AutoRTFM sanitizer failures are treated as warnings.
		autortfm_sanitizer_warn,

		// Enable AutoRTFM sanitizer.
		// AutoRTFM sanitizer failures are treated as fatal.
		autortfm_sanitizer_error,
	} autortfm_sanitizer_mode;

	// An enumerator of mutually exclusive AutoRTFM modes applied to functions or classes.
	typedef enum
	{
		// [[clang::autortfm(enable)]] / default
		// Generate AutoRTFM instrumentation for the function.
		autortfm_mode_enable = 0,

		// [[clang::autortfm(disable)]]
		// Do not generate AutoRTFM instrumentation for the function.
		// Calls to the function from AutoRTFM-enabled functions is a compile-time error.
		// WARNING: If you change this constant, then make sure to also update
		//          UEBuildModule.AddModuleToCompileEnvironment() in
		//          Engine\Source\Programs\UnrealBuildTool\Configuration\UEBuildModule.cs
		autortfm_mode_disable = 1,

		// [[clang::autortfm(disable_no_sanitize)]]
		// Same as autortfm_mode_disable, but the function will be ignored by the
		// AutoRTFM sanitizer. Note that the sanitizer will still be active for
		// function calls made by this function.
		autortfm_mode_disable_no_sanitize = 2,

		// [[clang::autortfm(infer)]]
		// AutoRTFM instrumentation mode is determined by the calls made by the function.
		autortfm_mode_infer = 3,

		// [[clang::autortfm(open)]] / [[clang::autortfm_always_open]]
		// Do not generate AutoRTFM instrumentation for the function.
		// Calling the function from the closed will call the uninstrumented variant.
		autortfm_mode_open = 4,

		// [[clang::autortfm(open_no_sanitize)]] / [[clang::autortfm_always_open_no_sanitize]]
		// Same as autortfm_mode_open, but the AutoRTFM sanitizer will not instrument the function.
		autortfm_mode_open_no_sanitize = 5,

		// [[clang::autortfm(autortfm_mode_internal)]]
		// Do not generate AutoRTFM instrumentation for the function.
		// Calls to the function from AutoRTFM-enabled functions will directly call
		// the open function without any pre / post callbacks or trampolines.
		// The AutoRTFM sanitizer ignore these functions - no calls to
		// autortfm_disable_sanitizer() or autortfm_enable_sanitizer() will be
		// emitted, nor will any calls to autortfm_sanitizer_open_write() be
		// emitted for writes, however calls to any open functions made by an
		// internal function may call autortfm_sanitizer_open_write().
		autortfm_mode_internal = 6,

		// [[clang::noautortfm]]
		// DEPRECATED - similar to Disable but does not cause a compile-time error if
		// called from an AutoRTFM-enabled function.
		autortfm_mode_no_autortfm = 7,
	} autortfm_mode;

	// The last mode declared in autortfm_mode. Used for range checking.
	static const autortfm_mode autortfm_last_mode = autortfm_mode_no_autortfm;

	// Returns the AutoRTFM mode as a string
	inline const char* autortfm_mode_to_string(autortfm_mode Mode)
	{
		switch (Mode)
		{
			case autortfm_mode_enable:
				return "autortfm_mode_enable";
			case autortfm_mode_disable:
				return "autortfm_mode_disable";
			case autortfm_mode_disable_no_sanitize:
				return "autortfm_mode_disable_no_sanitize";
			case autortfm_mode_infer:
				return "autortfm_mode_infer";
			case autortfm_mode_open:
				return "autortfm_mode_open";
			case autortfm_mode_open_no_sanitize:
				return "autortfm_mode_open_no_sanitize";
			case autortfm_mode_internal:
				return "autortfm_mode_internal";
			case autortfm_mode_no_autortfm:
				return "autortfm_mode_no_autortfm";
		}
	}

#ifdef __cplusplus
}
#endif

#ifdef __cplusplus

namespace AutoRTFM
{

// Returns true if mode is autortfm_mode_disable or autortfm_mode_disable_no_sanitize
constexpr inline bool IsDisabled(autortfm_mode Mode)
{
	return Mode == autortfm_mode_disable || Mode == autortfm_mode_disable_no_sanitize || Mode == autortfm_mode_no_autortfm;
}

// Returns true if mode is autortfm_mode_open or autortfm_mode_open_no_sanitize
constexpr inline bool IsOpen(autortfm_mode Mode)
{
	return Mode == autortfm_mode_open || Mode == autortfm_mode_open_no_sanitize;
}

// Returns true if mode suspends the AutoRTFM sanitizer (open_no_sanitize or disable_no_sanitize)
constexpr inline bool IsNoSanitize(autortfm_mode Mode)
{
	return Mode == autortfm_mode_open_no_sanitize || Mode == autortfm_mode_disable_no_sanitize;
}

}

namespace AutoRTFM::Constants
{
inline constexpr uint32_t Major = 0;
inline constexpr uint32_t Minor = 3;
inline constexpr uint32_t Patch = 0;

// The Magic Prefix constant - an arbitrarily chosen address prefix, shared
// between the AutoRTFM compiler and runtime.
// We add this prefix value to open function pointer addresses in our custom
// LLVM pass. At runtime, if we detect the Magic Prefix in the the top 16 bits
// of an open function pointer address, we assume that we can find a closed
// variant pointer residing 8 bytes before the function address.
inline constexpr uint64_t MagicPrefix = 0xa273'0000'0000'0000;
// Similar to the above, but these constants indicate that the low 48 bits
// provide a relative addresses from the open function to the closed
// function.
inline constexpr uint64_t PosOffsetMagicPrefix = 0xa272'0000'0000'0000;
inline constexpr uint64_t NegOffsetMagicPrefix = 0xa271'0000'0000'0000;

}  // namespace AutoRTFM::Constants
#endif
