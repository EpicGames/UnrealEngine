// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/Constants.h"

#if (defined(__AUTORTFM) && __AUTORTFM)
#define UE_AUTORTFM 1  // Compiler is 'verse-clang'
#else
#define UE_AUTORTFM 0
#endif

#if (defined(__AUTORTFM_ENABLED) && __AUTORTFM_ENABLED)
#define UE_AUTORTFM_ENABLED 1  // Compiled with '-fautortfm'
#else
#define UE_AUTORTFM_ENABLED 0
#endif

#if !defined(UE_AUTORTFM_ENABLED_RUNTIME_BY_DEFAULT)
#define UE_AUTORTFM_ENABLED_RUNTIME_BY_DEFAULT 1
#endif

#if !defined(UE_AUTORTFM_STATIC_VERIFIER)
#define UE_AUTORTFM_STATIC_VERIFIER 0
#endif

#if UE_AUTORTFM
// The annotated function will have no AutoRTFM closed variant generated, and
// cannot be called from another closed function. This attribute will eventually
// be deprecated and replaced with AUTORTFM_DISABLE.
#define UE_AUTORTFM_NOAUTORTFM [[clang::noautortfm]]

// Omits AutoRTFM instrumentation from the function. Calling this annotated
// function from the closed will automatically enter the open for the duration
// of the call and will return back to the closed.
// Deprecated: Use AUTORTFM_OPEN
#define UE_AUTORTFM_ALWAYS_OPEN [[clang::autortfm_always_open]]

// The same as UE_AUTORTFM_ALWAYS_OPEN, but disables the AutoRTFM sanitizer for
// the duration of this call.
// Deprecated: Use AUTORTFM_OPEN_NO_SANITIZE
#define UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION [[clang::autortfm_always_open_no_sanitize]]

// The same as UE_AUTORTFM_ALWAYS_OPEN, but disables the AutoRTFM sanitizer for
// the duration of this call.
// Deprecated: Use AUTORTFM_OPEN_NO_SANITIZE
#define UE_AUTORTFM_ALWAYS_OPEN_NO_SANITIZER [[clang::autortfm_always_open_no_sanitize]]

// Annotation that can be applied to classes, methods or functions to prevent
// AutoRTFM closed function(s) from being generated.
// Applying the annotation to a class is equivalent to applying the annotation
// to each method of the class.
// When annotating a function, the attribute must be placed on a the function
// declaration (usually in the header) and not a function implementation.
// Annotated functions cannot be called from another closed function and
// attempting to do so will result in a runtime failure.
#define AUTORTFM_DISABLE [[clang::autortfm(autortfm_mode_disable)]]

// Applies the AUTORTFM_DISABLE annotation if the condition argument evaluates to true.
#define AUTORTFM_DISABLE_IF(...) [[clang::autortfm(autortfm_mode_disable, __VA_ARGS__)]]

// Same as AUTORTFM_DISABLE, but the functions will be ignored by the
// AutoRTFM sanitizer.
#define AUTORTFM_DISABLE_NO_SANITIZE [[clang::autortfm(autortfm_mode_disable_no_sanitize)]]

// Begins a range where all classes and functions will be automatically
// annotated with AUTORTFM_DISABLE. Must be ended with AUTORTFM_DISABLE_END
// before the end of the file.
#define AUTORTFM_DISABLE_BEGIN _Pragma("clang attribute AutoRTFM_Disable.push (AUTORTFM_DISABLE, apply_to = any(function, record))")

// Ends a range started with AUTORTFM_DISABLE_BEGIN
#define AUTORTFM_DISABLE_END _Pragma("clang attribute AutoRTFM_Disable.pop")

// Annotation that can be applied to classes, methods or functions to re-enable
// AutoRTFM instrumentation which would otherwise be disabled by AUTORTFM_DISABLE.
// Useful for selectively enabling AutoRTFM on methods when a class is annotated
// AUTORTFM_DISABLE.
#define AUTORTFM_ENABLE [[clang::autortfm(autortfm_mode_enable)]]

// Applies the AUTORTFM_ENABLE annotation if the condition argument evaluates to true.
#define AUTORTFM_ENABLE_IF(...) [[clang::autortfm(autortfm_mode_enable, __VA_ARGS__)]]

// Annotation that can be applied to classes, methods or functions to infer
// whether AutoRTFM instrumentation should be enabled for each individual
// function based on the AutoRTFM-enabled state of each callee made by the
// function. If the function calls any AutoRTFM-disabled function, then the
// function will also be AutoRTFM-disabled, otherwise the function is
// AutoRTFM-enabled.
// Applying the annotation to a class is equivalent to applying the annotation
// to each method of the class.
// When annotating a function, the attribute must be placed on a the function
// declaration (usually in the header) and not a function implementation.
#define AUTORTFM_INFER [[clang::autortfm(autortfm_mode_infer)]]

// Annotation that can be applied to classes, methods or functions to prevent
// AutoRTFM closed function(s) from being generated. Unlike AUTORTFM_DISABLE
// annotated functions can be called from closed functions, which will call
// the uninstrumented function.
// Applying the annotation to a class is equivalent to applying the annotation
// to each method of the class.
// When annotating a function, the attribute must be placed on a the function
// declaration (usually in the header) and not a function implementation.
#define AUTORTFM_OPEN [[clang::autortfm(autortfm_mode_open)]]

// Similar to AUTORTFM_OPEN, but disables the AutoRTFM sanitizer on the call.
// Deprecated: Use AUTORTFM_OPEN_NO_SANITIZE
#define AUTORTFM_OPEN_NO_VALIDATION [[clang::autortfm(autortfm_mode_open_no_sanitize)]]

// Similar to AUTORTFM_OPEN, but disables AutoRTFM sanitizer on the call.
#define AUTORTFM_OPEN_NO_SANITIZE [[clang::autortfm(autortfm_mode_open_no_sanitize)]]

// Annotation that can be applied to function parameters to implicitly set the
// AutoRTFM mode of "naked" lambda arguments to AutoRTFM-enabled, regardless of the
// AutoRTFM mode of the caller function.
#define AUTORTFM_IMPLICIT_ENABLE [[clang::autortfm_implicit_mode(autortfm_mode_enable)]]

// Annotation that can be applied to function parameters to implicitly set the
// AutoRTFM mode of "naked" lambda arguments to AutoRTFM-disable, regardless of the
// AutoRTFM mode of the caller function.
#define AUTORTFM_IMPLICIT_DISABLE [[clang::autortfm_implicit_mode(autortfm_mode_disable)]]

// Evaluates to the AutoRTFM mode of function, method, type or class.
#define AUTORTFM_MODE_OF(EXPR_OR_TYPE) static_cast<autortfm_mode>(__autortfm_get_mode(EXPR_OR_TYPE))

// Evaluates to the AutoRTFM mode of function, method, type or class.
#define AUTORTFM_MODE_OF_CALL(CALL_EXPR) AUTORTFM_MODE_OF(__autortfm_declcall(CALL_EXPR))

// Force the call statement to be inlined.
#define UE_AUTORTFM_CALLSITE_FORCEINLINE [[clang::always_inline]]

// Evaluates to true iff the AutoRTFM sanitizer is enabled.
#define AUTORTFM_SANITIZER (__has_feature(autortfm_sanitizer))

// Provides a hint to the compiler that the expression is likely to be true
#define AUTORTFM_LIKELY(x) __builtin_expect(!!(x), 1)

// Provides a hint to the compiler that the expression is unlikely to be true
#define AUTORTFM_UNLIKELY(x) __builtin_expect(!!(x), 0)

#if AUTORTFM_SANITIZER
#define AUTORTFM_SANITIZER_ONLY(...) __VA_ARGS__
#else
#define AUTORTFM_SANITIZER_ONLY(...)
#endif

#else  // ^^^ UE_AUTORTFM ^^^ | vvv !UE_AUTORTFM vvv

#define UE_AUTORTFM_NOAUTORTFM
#define UE_AUTORTFM_ALWAYS_OPEN
#define UE_AUTORTFM_ALWAYS_OPEN_NO_MEMORY_VALIDATION
#define UE_AUTORTFM_ALWAYS_OPEN_NO_SANITIZER
#define AUTORTFM_DISABLE
#define AUTORTFM_DISABLE_IF(CONDITION)
#define AUTORTFM_DISABLE_NO_SANITIZE
#define AUTORTFM_DISABLE_BEGIN
#define AUTORTFM_DISABLE_END
#define AUTORTFM_ENABLE
#define AUTORTFM_INFER
#define AUTORTFM_OPEN
#define AUTORTFM_OPEN_NO_VALIDATION
#define AUTORTFM_OPEN_NO_SANITIZE
#define AUTORTFM_IMPLICIT_ENABLE
#define AUTORTFM_IMPLICIT_DISABLE
#define AUTORTFM_MODE_OF(EXPR_OR_TYPE) autortfm_mode_disable
#define AUTORTFM_MODE_OF_CALL(EXPR_OR_TYPE) autortfm_mode_disable
#define UE_AUTORTFM_CALLSITE_FORCEINLINE
#define AUTORTFM_SANITIZER (false)
#define AUTORTFM_LIKELY(x) (!!(x))
#define AUTORTFM_UNLIKELY(x) (!!(x))

#endif

#ifdef __cplusplus
#define AUTORTFM_NOEXCEPT noexcept
#define AUTORTFM_EXCEPT noexcept(false)
#else
#define AUTORTFM_NOEXCEPT
#define AUTORTFM_EXCEPT
#endif

#if UE_AUTORTFM && UE_AUTORTFM_STATIC_VERIFIER
#define UE_AUTORTFM_ENSURE_SAFE [[clang::autortfm_ensure_safe]]
#define UE_AUTORTFM_ASSUME_SAFE [[clang::autortfm_assume_safe]]
#else
#define UE_AUTORTFM_ENSURE_SAFE
#define UE_AUTORTFM_ASSUME_SAFE
#endif

#define UE_AUTORTFM_CONCAT_IMPL(A, B) A##B
#define UE_AUTORTFM_CONCAT(A, B) UE_AUTORTFM_CONCAT_IMPL(A, B)

#if defined(_MSC_VER)
#define UE_AUTORTFM_FORCEINLINE __forceinline
#define UE_AUTORTFM_FORCEINLINE_ALWAYS __forceinline
#define UE_AUTORTFM_FORCENOINLINE __declspec(noinline)
#define UE_AUTORTFM_ASSUME(x) __assume(x)
#elif defined(__clang__)
#define UE_AUTORTFM_FORCEINLINE __attribute__((always_inline)) inline
#define UE_AUTORTFM_FORCEINLINE_ALWAYS __attribute__((always_inline)) inline
#define UE_AUTORTFM_FORCENOINLINE __attribute__((noinline))
#define UE_AUTORTFM_ASSUME(x) __builtin_assume(x)
#else
#define UE_AUTORTFM_FORCEINLINE inline
#define UE_AUTORTFM_FORCEINLINE_ALWAYS inline
#define UE_AUTORTFM_FORCENOINLINE
#define UE_AUTORTFM_ASSUME(x)
#endif

#if (defined(UE_BUILD_DEBUG) && UE_BUILD_DEBUG) || (defined(AUTORTFM_BUILD_DEBUG) && AUTORTFM_BUILD_DEBUG)
// Force-inlining can make debugging glitchy. Disable this if we're running a debug build.
#undef UE_AUTORTFM_CALLSITE_FORCEINLINE
#define UE_AUTORTFM_CALLSITE_FORCEINLINE
#undef UE_AUTORTFM_FORCEINLINE
#define UE_AUTORTFM_FORCEINLINE inline
#endif

#ifdef _MSC_VER
#define AUTORTFM_DISABLE_UNREACHABLE_CODE_WARNINGS __pragma(warning(push)) __pragma(warning(disable : 4702)) /* unreachable code */
#define AUTORTFM_RESTORE_UNREACHABLE_CODE_WARNINGS __pragma(warning(pop))
#else
#define AUTORTFM_DISABLE_UNREACHABLE_CODE_WARNINGS
#define AUTORTFM_RESTORE_UNREACHABLE_CODE_WARNINGS
#endif

#define UE_AUTORTFM_UNUSED(UNUSEDVAR) (void)UNUSEDVAR

// It is critical that these functions are both static and forceinline to prevent binary sizes to explode
// This is a trick to ensure that there will never be a non-inlined version of these functions that the linker can decide to use
#ifndef UE_HEADER_UNITS
#define UE_AUTORTFM_CRITICAL_INLINE static UE_AUTORTFM_FORCEINLINE
#define UE_AUTORTFM_CRITICAL_INLINE_ALWAYS static UE_AUTORTFM_FORCEINLINE_ALWAYS
#else
#define UE_AUTORTFM_CRITICAL_INLINE \
	UE_AUTORTFM_FORCEINLINE  // TODO: This needs to be revisited. we don't want bloated executables when modules are enabled
#define UE_AUTORTFM_CRITICAL_INLINE_ALWAYS \
	UE_AUTORTFM_FORCEINLINE_ALWAYS  // TODO: This needs to be revisited. we don't want bloated executables when modules are enabled
#endif
