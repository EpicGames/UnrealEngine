// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/CAPI.h"

#include <cstdint>

namespace AutoRTFM
{

struct FAutoRTFMMetrics
{
	uint64_t NumTopLevelTransactionsCompleted;  // Tracks the number of top-level transactions that have been completed
	uint64_t NumTransactionsStarted;            // Tracks the number of transactions (at any depth) that have been started
	uint64_t NumTransactionsCommitted;          // Tracks the number of transactions (at any depth) that have been committed
	uint64_t NumTransactionsAborted;            // Tracks the number of transactions (at any depth) that have been aborted
	uint64_t NumRequestedAborts;                // Tracks the number of calls to AbortTransaction()
	uint64_t NumRequestedCascadingAborts;       // Tracks the number of calls to CascadingAbortTransaction()
	uint64_t NumRequestedCascadingRetries;      // Tracks the number of calls to CascadingRetryTransaction()
	uint64_t NumTransactionalViolations;        // Tracks the number of AutoRTFM aborts caused by violating transactional rules

	uint64_t OverallPeakMemoryUsage;            // Tracks the peak memory usage across all transactions, in bytes
	uint64_t ActiveTransactionPeakMemoryUsage;  // Tracks the peak memory usage within this transaction, in bytes

	// A log2-histogram of transaction memory usage, in bytes.
	// We do not distinguish between committed or aborted transactions in the histogram.
	// [0] indicates the number of finished transactions using between 0 and 1 bytes of memory.
	// [1] indicates the number of finished transactions using between 2 and 3 bytes of memory.
	// [2] indicates the number of finished transactions using between 4 and 7 bytes of memory.
	// [3] indicates the number of finished transactions using between 8 and 15 bytes of memory.
	// [4] indicates the number of finished transactions using between 16 and 31 bytes of memory.
	// and so on.
	using HistogramBucketArray = uint32_t[64];
	HistogramBucketArray MemoryUsageHistogram;

	static constexpr char CSVHeader[] =
		"NumTopLevelTransactionsCompleted,NumTransactionsStarted,NumTransactionsCommitted,"
		"NumTransactionsAborted,NumRequestedAborts,NumRequestedCascadingAborts,NumRequestedCascadingRetries,"
		"NumTransactionalViolations,OverallPeakMemoryUsage,ActiveTransactionPeakMemoryUsage,"
		"0b+,2b+,4b+,8b+,16b+,32b+,64b+,128b+,256b+,512b+,"            // Histogram[0..9]
		"1KB+,2KB+,4KB+,8KB+,16KB+,32KB+,64KB+,128KB+,256KB+,512KB+,"  // Histogram[10..19]
		"1MB+,2MB+,4MB+,8MB+,16MB+,32MB+,64MB+,128MB+,256MB+,512MB+,"  // Histogram[20..29]
		"1GB+,2GB+,4GB+,8GB+,16GB+,32GB+,64GB+,128GB+,256GB+,512GB+,"  // Histogram[30..39]
		"1TB+,2TB+,4TB+,8TB+,16TB+,32TB+,64TB+,128TB+,256TB+,512TB+,"  // Histogram[40..49]
		"1PB+,2PB+,4PB+,8PB+,16PB+,32PB+,64PB+,128PB+,256PB+,512PB+,"  // Histogram[50..59]
		"1024PB+,2048PB+,4096PB+,8192PB+";                             // Histogram[60..63]
};

/** Reset the internal metrics tracking back to zero. */
UE_AUTORTFM_API void ResetAutoRTFMMetrics();

/** Get a snapshot of the current internal metrics. */
UE_AUTORTFM_API FAutoRTFMMetrics GetAutoRTFMMetrics();

}
