// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include <utility>

namespace AutoRTFM
{

// An object wrapper that will ensure the copy constructor, move constructor,
// copy assignment, move assignment and destructor are all called in the open.
// This can be used to wrap a lambda capture if it is known that it safe and
// more optimal to copy, move & destruct the object in the open.
//
// For example - it may be safe to wrap a shared pointer with a TOpenWrapper
// when capturing for use by AutoRTFM::OnCommit() or AutoRTFM::OnAbort(), as
// the AutoRTFM::TTask destructor will be called regardless of a transactional
// failure or commit. This may be preferable to capturing the unwrapped shared
// pointer as we can avoid costly closed transactional logic to ensure the
// count is restored on failure:
//    AutoRTFM::OnCommit([WrappedSharedPtr = AutoRTFM::TOpenWrapper{MySharedPtr}]
//    {
//        WrappedSharedPtr.Object->DoSomething();
//    });
template <typename ObjectType>
class AUTORTFM_OPEN TOpenWrapper final
{
public:
	ObjectType Object;

	TOpenWrapper(const TOpenWrapper&) = default;
	TOpenWrapper(TOpenWrapper&&) = default;
	TOpenWrapper& operator=(const TOpenWrapper&) = default;
	TOpenWrapper& operator=(TOpenWrapper&&) = default;
	TOpenWrapper(const ObjectType& Value) : Object{Value} {}
	TOpenWrapper(ObjectType&& Value) : Object{std::move(Value)} {}
	~TOpenWrapper() = default;
};

}
