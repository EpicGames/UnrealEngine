// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/CAPI.h"
#include "AutoRTFM/Defines.h"

#if UE_AUTORTFM

namespace AutoRTFM::ForTheRuntime
{

enum class EHazardType
{
	CriticalSection,
	RWLock,
	Mutex,
	RecursiveMutex,
	SharedMutex,
	OpenFree,
	SpscQueue,
	MpscQueue,
};

/**
 * Prints an AutoRTFM Hazard banner, with descriptive text based on the hazard type, and a callstack.
 * Concludes with AutoRTFM::ReportError, which will terminate the process under normal circumstances.
 */
AUTORTFM_OPEN UE_AUTORTFM_API void ReportAutoRTFMHazard(EHazardType HazardType);

}

/**
 * Reports an AutoRTFM hazard if this block of code is reached from the closed.
 * `autortfm_is_closed` is optimized away at compile time, so this compiles away to nothing in an open function.
 */
#define UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(InHazardType)                                                                         \
	(autortfm_is_closed() ? ::AutoRTFM::ForTheRuntime::ReportAutoRTFMHazard(::AutoRTFM::ForTheRuntime::EHazardType::InHazardType) \
						  : (void)0)

#else

/** Our macro compiles away to nothing on compilers which do not support AutoRTFM. */
#define UE_AUTORTFM_REPORT_HAZARD_IF_CLOSED(InHazardType) ((void)0)

#endif  // UE_AUTORTFM