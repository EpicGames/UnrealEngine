// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/Defines.h"

#include <type_traits>

namespace AutoRTFM
{

// An enumerator of modes used to control the behaviour of values passed across
// the open -> closed boundary.
// For more information see the following declarations:
// * AutoRTFM::Open()
// * AutoRTFM::ReturnFromOpenModeFor
// * AutoRTFM::CHasAssignFromOpenToClosedMethod
// * AutoRTFM::CHasReturnFromOpenModeField
enum class EReturnFromOpenMode
{
	// The type does not support cross open -> closed boundaries.
	Unsupported,
	// The type can be copy-constructed in the closed from a value constructed
	// in the open.
	CopyConstructInClosed,
	// The type can be move-constructed in the closed from a value constructed
	// in the open.
	MoveConstructInClosed,
	// The type has a custom method that will be called to handle passing a
	// value of the type across a open -> closed boundary.
	// See AutoRTFM::CHasAssignFromOpenToClosedMethod.
	CustomMethod,
};

#if UE_AUTORTFM

// Concept that matches if the type T has a static field with the signature:
//    static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = <mode>;
template <typename T>
concept CHasReturnFromOpenModeField = std::is_same_v<decltype(T::AutoRTFMReturnFromOpenMode), const EReturnFromOpenMode>;

// Concept that matches if the type T has a static method with the signature:
//    static void AutoRTFMAssignFromOpenToClosed(T& Closed, U Open)
// Where `U` is `T`, `const T&` or `T&&`
// AutoRTFMAssignFromOpenToClosed is called in the open.
// Closed is a reference to a default-constructed T, which is constructed and
// destructed in the closed.
// Open is the return value from the open-lambda, which can be copied or moved.
template <typename T>
concept CHasAssignFromOpenToClosedMethod =
	std::is_same_v<decltype(T::AutoRTFMAssignFromOpenToClosed(std::declval<T&>(), std::declval<T>())), void>;

// Evaluates to the EReturnFromOpenMode for the given type.
// Evaluates to EReturnFromOpenMode::Unsupported, unless specialized.
template <typename T>
static constexpr EReturnFromOpenMode ReturnFromOpenModeFor = EReturnFromOpenMode::Unsupported;

// Specialization of ReturnFromOpenModeFor for types that have a static field with the signature:
//    static constexpr AutoRTFM::EReturnFromOpenMode AutoRTFMReturnFromOpenMode = <mode>;
// Evaluates to the field's mode.
// See CHasReturnFromOpenModeField.
template <CHasReturnFromOpenModeField T>
static constexpr EReturnFromOpenMode ReturnFromOpenModeFor<T> = T::AutoRTFMReturnFromOpenMode;

// Specialization of ReturnFromOpenModeFor for types that have a static mode with the signature:
//    static void AutoRTFMAssignFromOpenToClosed(T& Closed, U Open)
// Where `U` is `T`, `const T&` or `T&&`
// Evaluates to EReturnFromOpenMode::CustomMethod.
// See CHasAssignFromOpenToClosedMethod.
template <CHasAssignFromOpenToClosedMethod T>
static constexpr EReturnFromOpenMode ReturnFromOpenModeFor<T> = EReturnFromOpenMode::CustomMethod;

// Specialization of ReturnFromOpenModeFor for fundamental types.
// Evaluates to EReturnFromOpenMode::CopyConstructInClosed.
template <typename T>
requires(std::is_fundamental_v<T>)
static constexpr EReturnFromOpenMode ReturnFromOpenModeFor<T> = EReturnFromOpenMode::CopyConstructInClosed;

// Specialization of ReturnFromOpenModeFor for raw pointer types.
// Evaluates to EReturnFromOpenMode::CopyConstructInClosed.
template <typename T>
static constexpr EReturnFromOpenMode ReturnFromOpenModeFor<T*> = EReturnFromOpenMode::CopyConstructInClosed;

#endif  // UE_AUTORTFM

}
