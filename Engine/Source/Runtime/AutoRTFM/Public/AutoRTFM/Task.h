// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cstddef>
#include <type_traits>
#include <utility>

#include "AutoRTFM/Defines.h"
#include "CAPI.h"
#include "Constants.h"

namespace AutoRTFM
{

// TTask is a small std::function like class that can wrap a functor (lambda, etc).
// TTask is used by AutoRTFM for callbacks such as OnCommit and OnAbort
// handlers, which there may be many accumulating throughout a transaction.
// For this reason TTask is designed to be compact in size (smaller than
// TFunction), but large enough to fit the most common functors used by AutoRTFM
// tasks.
// Note: to prevent generating closed-variants of task-wrapped functions,
// operator() cannot be called from the closed. All other operations on
// the task can be called from the open or closed.
template <typename Signature>
class TTask;

template <typename ReturnType, typename... ParameterTypes>
class TTask<ReturnType(ParameterTypes...)>
{
public:
	static constexpr size_t InlineDataSize = 16;
	static constexpr size_t InlineDataAlignment = 8;

private:
	// The payload holds either the inline function data, or a pointer to the
	// data in a heap allocation.
	union FPayload
	{
		std::byte Inline[InlineDataSize];
		void* External;
	};

	// Holds the function pointers for operating on the payload.
	struct FPayloadMethods
	{
		using CallFn = ReturnType(FPayload& Payload, ParameterTypes...);
		using CopyFn = void(FPayload& Dst, const FPayload& Src);
		using MoveFn = void(FPayload& Dst, FPayload& Src);
		using DestructFn = void(FPayload& Payload);

		CallFn* Call = nullptr;          // Calls the function held by the payload.
		CopyFn* Copy = nullptr;          // Copies the payload.
		MoveFn* Move = nullptr;          // Moves the payload.
		DestructFn* Destruct = nullptr;  // Destructs the payload.
	};

	// Traits helper for a functor (T).
	template <typename T>
	struct TTraits
	{
		// True iff T can fit within the inline payload, and does not require
		// courser alignment than provided by the inline payload.
		static constexpr bool bCanUseInline = sizeof(T) <= InlineDataSize && alignof(T) <= InlineDataAlignment;

		// True iff the copy constructor of the functor T is AutoRTFM-disabled.
		static constexpr bool bCopyDisabled = AutoRTFM::IsDisabled(AUTORTFM_MODE_OF_CALL(::new T(std::declval<const T&>())));

		// True iff the move constructor of the functor T is AutoRTFM-disabled.
		static constexpr bool bMoveDisabled = AutoRTFM::IsDisabled(AUTORTFM_MODE_OF_CALL(::new T(std::declval<T&&>())));

		// True iff the destructor of the functor T is AutoRTFM-disabled.
		static constexpr bool bDestructorDisabled = AutoRTFM::IsDisabled(AUTORTFM_MODE_OF_CALL(std::declval<T>().~T()));

		// Returns the functor from the payload.
		static T* FunctionFrom(FPayload& Payload)
		{
			return bCanUseInline ? reinterpret_cast<T*>(Payload.Inline) : reinterpret_cast<T*>(Payload.External);
		}

		// Returns the functor from the payload.
		static const T* FunctionFrom(const FPayload& Payload)
		{
			return bCanUseInline ? reinterpret_cast<const T*>(Payload.Inline) : reinterpret_cast<const T*>(Payload.External);
		}

		// Calls the function held by the payload. Cannot be called from the closed.
		AUTORTFM_DISABLE static ReturnType Call(FPayload& Payload, ParameterTypes... Arguments)
		{
			return (*FunctionFrom(Payload))(std::forward<ParameterTypes>(Arguments)...);
		}

		// Copies the payload.
		static void Copy(FPayload& Dst, const FPayload& Src)
		{
			if constexpr (bCopyDisabled)
			{
				if (autortfm_is_closed())
				{
					autortfm_unreachable("TTask functor has an AutoRTFM-disabled copy constructor which was called from the closed");
				}
			}
			if constexpr (bCanUseInline)
			{
				new (Dst.Inline) T(*FunctionFrom(Src));
			}
			else
			{
				Dst.External = new T(*FunctionFrom(Src));
			}
		}

		// Moves the payload.
		static void Move(FPayload& Dst, FPayload& Src)
		{
			if constexpr (bMoveDisabled)
			{
				if (autortfm_is_closed())
				{
					autortfm_unreachable("TTask functor has an AutoRTFM-disabled move constructor which was called from the closed");
				}
			}
			if constexpr (bCanUseInline)
			{
				new (Dst.Inline) T(std::move(*FunctionFrom(Src)));
			}
			else
			{
				Dst.External = Src.External;
			}
		}

		// Destructs the payload.
		static void Destruct(FPayload& Payload)
		{
			if constexpr (bDestructorDisabled)
			{
				if (autortfm_is_closed())
				{
					autortfm_unreachable("TTask functor has an AutoRTFM-disabled destructor which was called from the closed");
				}
			}
			if constexpr (bCanUseInline)
			{
				FunctionFrom(Payload)->~T();
			}
			else
			{
				delete FunctionFrom(Payload);
			}
		}

		// The methods to operate on a payload for a functor of type T.
		static constexpr FPayloadMethods PayloadMethods{&Call, &Copy, &Move, &Destruct};
	};

public:
	// Constructor.
	// The TTask is constructed in a non-set state.
	TTask() = default;

	// Destructor.
	~TTask()
	{
		Reset();
	}

	// Copy constructor
	TTask(const TTask& Other)
	{
		if (Other.IsSet())
		{
			PayloadMethods = Other.PayloadMethods;
			PayloadMethods->Copy(Payload, Other.Payload);
		}
	}

	// Move constructor
	TTask(TTask&& Other)
	{
		if (Other.IsSet())
		{
			PayloadMethods = Other.PayloadMethods;
			PayloadMethods->Move(Payload, Other.Payload);
			Other.PayloadMethods = nullptr;
		}
	}

	// Constructor from functor.
	template <typename FunctorType, typename = std::enable_if_t<
										// Use the explicit TTask copy / move constructors instead of this constructor
										!std::is_same_v<std::decay_t<FunctorType>, TTask> &&
										// The templated argument must be a functor with the correct signature
										std::is_invocable_r_v<ReturnType, std::decay_t<FunctorType>, ParameterTypes...>>>
	TTask(FunctorType&& Functor)
	{
		using Decayed = std::decay_t<FunctorType>;
		using Traits = TTraits<Decayed>;
		static_assert(std::is_same_v<std::invoke_result_t<Decayed, ParameterTypes...>, ReturnType>,
			"Functor return type does not match TTask return type");
		PayloadMethods = &Traits::PayloadMethods;
		if (Traits::bCanUseInline)
		{
			new (Payload.Inline) Decayed(std::forward<FunctorType>(Functor));
		}
		else
		{
			Payload.External = new Decayed(std::forward<FunctorType>(Functor));
		}
	}

	// Copy assignment operator
	TTask& operator=(const TTask& Other)
	{
		if (&Other != this)
		{
			Reset();
			if (Other.IsSet())
			{
				PayloadMethods = Other.PayloadMethods;
				PayloadMethods->Copy(Payload, Other.Payload);
			}
		}
		return *this;
	}

	// Move assignment operator
	TTask& operator=(TTask&& Other)
	{
		if (&Other != this)
		{
			Reset();
			if (Other.IsSet())
			{
				PayloadMethods = Other.PayloadMethods;
				PayloadMethods->Move(Payload, Other.Payload);
				Other.PayloadMethods = nullptr;
			}
		}
		return *this;
	}

	// Call operator. Cannot be called from the closed.
	template <typename... ArgumentTypes>
	AUTORTFM_DISABLE ReturnType operator()(ArgumentTypes&&... Arguments) const
	{
		return PayloadMethods->Call(Payload, std::forward<ArgumentTypes>(Arguments)...);
	}

	// Reset the task to an unset state.
	void Reset()
	{
		if (PayloadMethods)
		{
			PayloadMethods->Destruct(Payload);
			PayloadMethods = nullptr;
		}
	}

	// Returns true if the task holds a functor.
	bool IsSet() const
	{
		return PayloadMethods != nullptr;
	}

	// Returns the address of the functor.
	void* FunctionAddress() const
	{
		return reinterpret_cast<void*>(PayloadMethods->Call);
	}

private:
	// mutable as the functor might be mutable.
	alignas(InlineDataAlignment) mutable FPayload Payload;
	FPayloadMethods const* PayloadMethods = nullptr;
};

// Tasks are designed to be compact.
static_assert(sizeof(TTask<void()>) <= 24);

}  // namespace AutoRTFM
