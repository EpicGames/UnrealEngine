// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include "AutoRTFM/Task.h"

#if defined(UE_BUILD_SHIPPING) && UE_BUILD_SHIPPING
#define AUTORTFM_ENABLE_TEST_UTILS 0
#else
#define AUTORTFM_ENABLE_TEST_UTILS 1
#endif

namespace AutoRTFM::Testing
{

// Implemented by the test framework, and called when AUTORTFM_TESTING_ASSERT() fails.
void AssertionFailure(const char* Expression, const char* File, int Line);

#define AUTORTFM_TESTING_ASSERT(Condition)                                                   \
	do                                                                                       \
	{                                                                                        \
		if (!(Condition))                                                                    \
		{                                                                                    \
			AssertionFailure("AUTORTFM_TESTING_ASSERT(" #Condition ")", __FILE__, __LINE__); \
		}                                                                                    \
	} while (false)

// Run the callback in a transaction like Transact, but abort program
// execution if the result is anything other than autortfm_committed.
// Useful for testing.
template <typename TFunctor>
static UE_AUTORTFM_FORCEINLINE void Commit(AUTORTFM_IMPLICIT_ENABLE const TFunctor& Functor)
{
	const ETransactionResult Result = Transact(Functor);
	AUTORTFM_TESTING_ASSERT(ETransactionResult::Committed == Result);
}

// Run the callback in a transaction like Transact, but abort program
// execution if the result is anything other than abort.
template <typename TFunctor>
static UE_AUTORTFM_FORCEINLINE void Abort(AUTORTFM_IMPLICIT_ENABLE const TFunctor& Functor)
{
	const ETransactionResult Result = Transact(Functor);
	AUTORTFM_TESTING_ASSERT(HasAborted(Result));
}

// Force set the AutoRTFM runtime state. Used in testing. Returns the old value.
UE_AUTORTFM_API ForTheRuntime::EAutoRTFMEnabledState ForceSetAutoRTFMRuntime(ForTheRuntime::EAutoRTFMEnabledState State);

struct FEnabledStateResetterScoped final
{
	FEnabledStateResetterScoped(ForTheRuntime::EAutoRTFMEnabledState State) : Original(ForceSetAutoRTFMRuntime(State)) {}
	~FEnabledStateResetterScoped()
	{
		ForceSetAutoRTFMRuntime(Original);
	}

private:
	const ForTheRuntime::EAutoRTFMEnabledState Original;
};

#if AUTORTFM_ENABLE_TEST_UTILS && UE_AUTORTFM

// A utility for ensuring that allocations made by AutoRTFM are not leaked and
// free / reallocations pass the same size as the allocation.
struct AUTORTFM_DISABLE FTrackingAllocator
{
	using OnFailureFn = void (*)(const char* Error);

	// Constructor.
	// OnFailure is a callback function called to report an allocation error.
	// Only a single FTrackingAllocator can be alive at any given time.
	UE_AUTORTFM_API FTrackingAllocator(OnFailureFn OnFailure);

	// Destructor.
	// Must only be called after AutoRTFM has been shutdown.
	UE_AUTORTFM_API ~FTrackingAllocator();

	// Replaces the allocator external API functions of API to install the
	// tracking allocator.
	// If bRecordAllocationStackTraces is true, then the stacktrace of each
	// heap allocation will be recorded, so that allocation failures and leaks
	// can be easily identified.
	UE_AUTORTFM_API void Install(autortfm_extern_api& API, bool bRecordAllocationStackTraces = false);

	// Returns the total number of bytes allocated by AutoRTFM.
	UE_AUTORTFM_API size_t TotalBytesAllocated() const;

	// Prints callstack of active allocations.
	// Requires FTrackingAllocator to be constructed with bRecordAllocationStackTraces enabled.
	UE_AUTORTFM_API void PrintAllocationCallstacks(size_t MaxCount = 5) const;

private:
	struct FState;  // PIMPL internal state
	FState* State = nullptr;
};

#endif  // AUTORTFM_ENABLE_TEST_UTILS && UE_AUTORTFM

#if AUTORTFM_ENABLE_TEST_UTILS

struct FCrashInfo
{
	// A description of the cause of the crash
	const char* Kind = nullptr;
	// The program counter at the point of the crash. May be null.
	void* ProgramCounter = nullptr;
	// The memory address that caused the crash. May be null.
	void* Address = nullptr;
};

// Sets up a exception / signal handler to call Callback on uncaught exception / fault.
// Can only be called once for the lifetime of the process.
AUTORTFM_DISABLE UE_AUTORTFM_API void OnCrash(TTask<void(const FCrashInfo&)> Callback);

#endif  // AUTORTFM_ENABLE_TEST_UTILS

#undef AUTORTFM_TESTING_ASSERT

}  // namespace AutoRTFM::Testing
