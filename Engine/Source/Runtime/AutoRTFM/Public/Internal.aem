# Copyright Epic Games, Inc. All Rights Reserved.

# AutoRTFM internal mappings.
# See Engine/Source/Runtime/AutoRTFM/Documentation/ExternalMappings.md
# for syntax and more information.

same autortfm_lookup_function
