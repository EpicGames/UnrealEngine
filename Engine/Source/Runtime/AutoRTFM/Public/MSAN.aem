# Copyright Epic Games, Inc. All Rights Reserved.

# AutoRTFM external mappings for ASAN.
# See Engine/Source/Runtime/AutoRTFM/Documentation/ExternalMappings.md
# for syntax and more information.

same __msan_set_origin
same __msan_get_origin
same __msan_origin_is_descendant_or_same
same __msan_get_track_origins
same __msan_get_umr_origin
same __msan_unpoison
same __msan_unpoison_string
same __msan_unpoison_param
same __msan_poison
same __msan_partial_poison
same __msan_test_shadow
same __msan_check_mem_is_initialized
same __msan_set_expect_umr
same __msan_set_keep_going
same __msan_print_shadow
same __msan_dump_shadow
same __msan_has_dynamic_component
same __msan_allocated_memory
same __msan_default_options
same __msan_set_death_callback
same __msan_copy_shadow
same __msan_scoped_disable_interceptor_checks
same __msan_scoped_enable_interceptor_checks
same __msan_start_switch_fiber
same __msan_finish_switch_fiber
same __msan_init
same __msan_warning
same __msan_warning_noreturn
same __msan_warning_with_origin
same __msan_warning_with_origin_noreturn
same __msan_maybe_warning_1
same __msan_maybe_warning_2
same __msan_maybe_warning_4
same __msan_maybe_warning_8
same __msan_maybe_store_origin_1
same __msan_maybe_store_origin_2
same __msan_maybe_store_origin_4
same __msan_maybe_store_origin_8
same __msan_clear_and_unpoison
same __msan_memcpy
same __msan_memset
same __msan_memmove
same __msan_poison_stack
# same __msan_load_unpoisoned
same __msan_set_alloca_origin
same __msan_set_alloca_origin4
same __msan_set_alloca_origin_with_descr
same __msan_set_alloca_origin_no_descr
same __msan_chain_origin
same __msan_clear_on_return
same __msan_set_poison_in_malloc
same __sanitizer_dtor_callback
same __sanitizer_dtor_callback_fields
same __sanitizer_dtor_callback_vptr
