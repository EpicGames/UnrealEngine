# Copyright Epic Games, Inc. All Rights Reserved.

# AutoRTFM external mappings for new, new[], delete and delete[].
# See Engine/Source/Runtime/AutoRTFM/Documentation/ExternalMappings.md
# for syntax and more information.

map  _Znwm -> RTFM_Znwm
map  _Znam -> RTFM_Znam
map  _ZnwmRKSt9nothrow_t -> RTFM_ZnwmRKSt9nothrow_t
map  _ZnamRKSt9nothrow_t -> RTFM_ZnamRKSt9nothrow_t
map  _ZnwmSt11align_val_t -> RTFM_ZnwmSt11align_val_t
map  _ZnamSt11align_val_t -> RTFM_ZnamSt11align_val_t
map  _ZnwmSt11align_val_tRKSt9nothrow_t -> RTFM_ZnwmSt11align_val_tRKSt9nothrow_t
map  _ZnamSt11align_val_tRKSt9nothrow_t -> RTFM_ZnamSt11align_val_tRKSt9nothrow_t
map  _ZdlPv -> RTFM_ZdlPv
map  _ZdaPv -> RTFM_ZdaPv
map  _ZdlPvRKSt9nothrow_t -> RTFM_ZdlPvRKSt9nothrow_t
map  _ZdaPvRKSt9nothrow_t -> RTFM_ZdaPvRKSt9nothrow_t
map  _ZdlPvm -> RTFM_ZdlPvm
map  _ZdaPvm -> RTFM_ZdaPvm
map  _ZdlPvSt11align_val_t -> RTFM_ZdlPvSt11align_val_t
map  _ZdaPvSt11align_val_t -> RTFM_ZdaPvSt11align_val_t
map  _ZdlPvSt11align_val_tRKSt9nothrow_t -> RTFM_ZdlPvSt11align_val_tRKSt9nothrow_t
map  _ZdaPvSt11align_val_tRKSt9nothrow_t -> RTFM_ZdaPvSt11align_val_tRKSt9nothrow_t
map  _ZdlPvmSt11align_val_t -> RTFM_ZdlPvmSt11align_val_t
map  _ZdaPvmSt11align_val_t -> RTFM_ZdaPvmSt11align_val_t
