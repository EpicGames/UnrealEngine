# Copyright Epic Games, Inc. All Rights Reserved.

# AutoRTFM external mappings for new, new[], delete and delete[].
# See Engine/Source/Runtime/AutoRTFM/Documentation/ExternalMappings.md
# for syntax and more information.

map  ??2@YAPEAX_K@Z -> RTFM_Znwm
map  ??_U@YAPEAX_K@Z -> RTFM_Znam
map  ??2@YAPEAX_KAEBUnothrow_t@std@@@Z -> RTFM_ZnwmRKSt9nothrow_t
map  ??_U@YAPEAX_KAEBUnothrow_t@std@@@Z -> RTFM_ZnamRKSt9nothrow_t
map  ??2@YAPEAX_KW4align_val_t@std@@@Z -> RTFM_ZnwmSt11align_val_t
map  ??_U@YAPEAX_KW4align_val_t@std@@@Z -> RTFM_ZnamSt11align_val_t
map  ??2@YAPEAX_KW4align_val_t@std@@AEBUnothrow_t@1@@Z -> RTFM_ZnwmSt11align_val_tRKSt9nothrow_t
map  ??_U@YAPEAX_KW4align_val_t@std@@AEBUnothrow_t@1@@Z -> RTFM_ZnamSt11align_val_tRKSt9nothrow_t
map  ??3@YAXPEAX@Z -> RTFM_ZdlPv
map  ??_V@YAXPEAX@Z -> RTFM_ZdaPv
map  ??3@YAXPEAXAEBUnothrow_t@std@@@Z -> RTFM_ZdlPvRKSt9nothrow_t
map  ??_V@YAXPEAXAEBUnothrow_t@std@@@Z -> RTFM_ZdaPvRKSt9nothrow_t
map  ??3@YAXPEAX_K@Z -> RTFM_ZdlPvm
map  ??_V@YAXPEAX_K@Z -> RTFM_ZdaPvm
map  ??3@YAXPEAXW4align_val_t@std@@@Z -> RTFM_ZdlPvSt11align_val_t
map  ??_V@YAXPEAXW4align_val_t@std@@@Z -> RTFM_ZdaPvSt11align_val_t
map  ??3@YAXPEAXW4align_val_t@std@@AEBUnothrow_t@1@@Z -> RTFM_ZdlPvSt11align_val_tRKSt9nothrow_t
map  ??_V@YAXPEAXW4align_val_t@std@@AEBUnothrow_t@1@@Z -> RTFM_ZdaPvSt11align_val_tRKSt9nothrow_t
map  ??3@YAXPEAX_KW4align_val_t@std@@@Z -> RTFM_ZdlPvmSt11align_val_t
map  ??_V@YAXPEAX_KW4align_val_t@std@@@Z -> RTFM_ZdaPvmSt11align_val_t
