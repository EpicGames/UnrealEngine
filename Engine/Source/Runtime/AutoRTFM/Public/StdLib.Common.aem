# Copyright Epic Games, Inc. All Rights Reserved.

# Common AutoRTFM external mappings.
# See Engine/Source/Runtime/AutoRTFM/Documentation/ExternalMappings.md
# for syntax and more information.

# Standard library
same acos
same acosf
same acosh
same acoshf
same asin
same asinf
same asinh
same asinhf
same atan
same atan2
same atan2f
same atanf
same atanh
same atanhf
map  atexit -> RTFM_atexit
map  calloc -> RTFM_calloc
same cos
same cosf
same cosh
same coshf
same exp
same expf
same fflush
same fmod
same fmodf
map  free -> RTFM_free
same iswalnum
same iswalpha
same iswcntrl
same iswdigit
same iswgraph
same iswlower
same iswprint
same iswpunct
same iswspace
same iswupper
same iswxdigit
same llrint
same llrintf
same log
same logf
map  malloc -> RTFM_malloc
same memcmp
map  memcpy -> RTFM_memcpy
map  memmove -> RTFM_memmove
map  memset -> RTFM_memset
same modf
same modff
same pow
same powf
same puts
same rand
map  realloc -> RTFM_realloc
same sin
same sinf
same sinh
same sinhf
same sqrt
same sqrtf
map  strcat -> RTFM_strcat
same strchr
same strcmp
map  strcpy -> RTFM_strcpy
same strlen
map  strncat -> RTFM_strncat
same strncmp
map  strncpy -> RTFM_strncpy
same strrchr
same strstr
map  strtod -> RTFM_strtod
map  strtof -> RTFM_strtof
map  strtol -> RTFM_strtol
map  strtoll -> RTFM_strtoll
map  strtoul -> RTFM_strtoul
map  strtoull -> RTFM_strtoull
same tan
same tanf
same tanh
same tanhf
same wcschr
same wcscmp
map  wcscpy -> RTFM_wcscpy
same wcslen
same wcsncmp
map  wcsncpy -> RTFM_wcsncpy
same wcsnlen
same wcsrchr
same wcsstr
map  wcstod -> RTFM_wcstod
map  wcstof -> RTFM_wcstof
map  wcstol -> RTFM_wcstol
map  wcstold -> RTFM_wcstold
map  wcstoll -> RTFM_wcstoll
map  wcstoul -> RTFM_wcstoul
map  wcstoull -> RTFM_wcstoull
