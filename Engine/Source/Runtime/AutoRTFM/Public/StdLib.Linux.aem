# Copyright Epic Games, Inc. All Rights Reserved.

# Linux-specific AutoRTFM external mappings.
# See Engine/Source/Runtime/AutoRTFM/Documentation/ExternalMappings.md
# for syntax and more information.

# Standard library
map  __fxstat -> RTFM___fxstat
map  __xstat -> RTFM___xstat
same _ZNSt3__112__next_primeEm
same bcmp
same clock_gettime
same fmodl
same getpid
same gettimeofday
same gmtime_r
same localtime_r
same modfl
map  printf -> RTFM_printf
same pthread_getspecific
same pthread_self
same putc
map  strtof32 -> RTFM_strtof32
map  strtof64 -> RTFM_strtof64
map  wprintf -> RTFM_wprintf
map  snprintf -> RTFM_snprintf
map  swprintf -> RTFM_swprintf
map  vsnprintf -> RTFM_vsnprintf
map  vswprintf -> RTFM_vswprintf

# Exceptions
same __cxa_allocate_exception
same __cxa_throw
same __cxa_begin_catch
same __cxa_end_catch

# std::to_chars
map _ZNSt3__18to_charsEPcS0_d -> RTFM__ToChars_d
map _ZNSt3__18to_charsEPcS0_dNS_12chars_formatE -> RTFM__ToChars_d_F
map _ZNSt3__18to_charsEPcS0_dNS_12chars_formatEi -> RTFM__ToChars_d_FP
map _ZNSt3__18to_charsEPcS0_e -> RTFM__ToChars_ld
map _ZNSt3__18to_charsEPcS0_eNS_12chars_formatE -> RTFM__ToChars_ld_F
map _ZNSt3__18to_charsEPcS0_eNS_12chars_formatEi -> RTFM__ToChars_ld_FP
map _ZNSt3__18to_charsEPcS0_f -> RTFM__ToChars_f
map _ZNSt3__18to_charsEPcS0_fNS_12chars_formatE -> RTFM__ToChars_f_F
map _ZNSt3__18to_charsEPcS0_fNS_12chars_formatEi -> RTFM__ToChars_f_FP
