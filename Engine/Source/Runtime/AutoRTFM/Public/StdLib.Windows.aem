# Copyright Epic Games, Inc. All Rights Reserved.

# Windows-specific AutoRTFM external mappings.
# See Engine/Source/Runtime/AutoRTFM/Documentation/ExternalMappings.md
# for syntax and more information.

# Standard library
map  __acrt_iob_func -> RTFM___acrt_iob_func
same __imp_acos
same __imp_acosf
same __imp_acosh
same __imp_acoshf
same __imp_asin
same __imp_asinf
same __imp_asinh
same __imp_asinhf
same __imp_atan
same __imp_atan2
same __imp_atan2f
same __imp_atanf
same __imp_atanh
same __imp_atanhf
same __imp_cos
same __imp_cosf
same __imp_cosh
same __imp_coshf
same __imp_exp
same __imp_expf
same __imp_fmod
same __imp_fmodf
same __imp_iswalnum
same __imp_iswalpha
same __imp_iswcntrl
same __imp_iswdigit
same __imp_iswgraph
same __imp_iswlower
same __imp_iswprint
same __imp_iswpunct
same __imp_iswspace
same __imp_iswupper
same __imp_iswxdigit
same __imp_llrint
same __imp_llrintf
same __imp_log
same __imp_logf
same __imp_memcmp
same __imp_modf
same __imp_modff
same __imp_pow
same __imp_powf
same __imp_rand
same __imp_sin
same __imp_sinf
same __imp_sinh
same __imp_sinhf
same __imp_sqrt
same __imp_sqrtf
same __imp_strchr
same __imp_strcmp
same __imp_strlen
same __imp_strncmp
same __imp_strrchr
same __imp_strstr
same __imp_tan
same __imp_tanf
same __imp_tanh
same __imp_tanhf
same __imp_wcschr
same __imp_wcscmp
same __imp_wcslen
same __imp_wcsncmp
same __imp_wcsstr
map  __std_swap_ranges_trivially_swappable_noalias -> RTFM___std_swap_ranges_trivially_swappable_noalias
map  __stdio_common_vfprintf -> RTFM___stdio_common_vfprintf
map  __stdio_common_vfwprintf -> RTFM___stdio_common_vfwprintf
map  __stdio_common_vsprintf -> RTFM___stdio_common_vsprintf
map  __stdio_common_vswprintf -> RTFM___stdio_common_vswprintf
same _dtest
same _fdtest
same _finite
same _isnan
same _iswalnum_l
same _iswalpha_l
same _iswcntrl_l
same _iswdigit_l
same _iswgraph_l
same _iswlower_l
same _iswprint_l
same _iswpunct_l
same _iswspace_l
same _iswupper_l
same _iswxdigit_l
same _ldtest
map  _strtoi64 -> RTFM__strtoi64
same _tcslen
same _tcsncmp
# same _tcsnlen
map  _wcstod_l -> RTFM__wcstod_l
map  _wcstof_l -> RTFM__wcstof_l
map  _wcstoi64 -> RTFM__wcstoi64
map  _wcstoi64_l -> RTFM__wcstoi64_l
map  _wcstol_l -> RTFM__wcstol_l
map  _wcstold_l -> RTFM__wcstold_l
map  _wcstoll_l -> RTFM__wcstoll_l
map  _wcstoui64 -> RTFM__wcstoui64
map  _wcstoui64_l -> RTFM__wcstoui64_l
map  _wcstoul_l -> RTFM__wcstoul_l
map  _wcstoull_l -> RTFM__wcstoull_l
same _wtof
same _wtof_l
same _wtoi
same _wtoi_l
same _wtoi64
same _wtoi64_l
same _wtol
same _wtol_l
same _wtoll
same _wtoll_l
same atof
same atoi
same putchar

# Win32 APIs
same _Query_perf_counter
same _Query_perf_frequency
same FlsGetValue
map  FlsSetValue -> RTFM_FlsSetValue
same GetCurrentThreadId
same GetLocalTime
same GetSystemTime
same IsDebuggerPresent
same GetCurrentProcessId
same GetFileAttributesW
same QueryPerformanceCounter
same QueryPerformanceFrequency
same TlsGetValue
map  TlsSetValue -> RTFM_TlsSetValue

# Exceptions
same _CxxThrowException
