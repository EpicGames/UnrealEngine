# Copyright Epic Games, Inc. All Rights Reserved.

# AutoRTFM external mappings for UBSAN.
# See Engine/Source/Runtime/AutoRTFM/Documentation/ExternalMappings.md
# for syntax and more information.

same __ubsan_handle_add_overflow
same __ubsan_handle_add_overflow_abort
same __ubsan_handle_alignment_assumption
same __ubsan_handle_alignment_assumption_abort
same __ubsan_handle_builtin_unreachable
same __ubsan_handle_cfi_bad_type
same __ubsan_handle_cfi_check_fail
same __ubsan_handle_cfi_check_fail_abort
same __ubsan_handle_divrem_overflow
same __ubsan_handle_divrem_overflow_abort
same __ubsan_handle_dynamic_type_cache_miss
same __ubsan_handle_dynamic_type_cache_miss_abort
same __ubsan_handle_float_cast_overflow
same __ubsan_handle_float_cast_overflow_abort
same __ubsan_handle_function_type_mismatch
same __ubsan_handle_function_type_mismatch_abort
same __ubsan_handle_implicit_conversion
same __ubsan_handle_implicit_conversion_abort
same __ubsan_handle_invalid_builtin
same __ubsan_handle_invalid_builtin_abort
same __ubsan_handle_invalid_objc_cast
same __ubsan_handle_invalid_objc_cast_abort
same __ubsan_handle_load_invalid_value
same __ubsan_handle_load_invalid_value_abort
same __ubsan_handle_missing_return
same __ubsan_handle_mul_overflow
same __ubsan_handle_mul_overflow_abort
same __ubsan_handle_negate_overflow
same __ubsan_handle_negate_overflow_abort
same __ubsan_handle_nonnull_arg
same __ubsan_handle_nonnull_arg_abort
same __ubsan_handle_nonnull_return_v1
same __ubsan_handle_nonnull_return_v1_abort
same __ubsan_handle_nullability_arg
same __ubsan_handle_nullability_arg_abort
same __ubsan_handle_nullability_return_v1
same __ubsan_handle_nullability_return_v1_abort
same __ubsan_handle_out_of_bounds
same __ubsan_handle_out_of_bounds_abort
same __ubsan_handle_local_out_of_bounds
same __ubsan_handle_local_out_of_bounds_abort
same __ubsan_handle_pointer_overflow
same __ubsan_handle_pointer_overflow_abort
same __ubsan_handle_shift_out_of_bounds
same __ubsan_handle_shift_out_of_bounds_abort
same __ubsan_handle_sub_overflow
same __ubsan_handle_sub_overflow_abort
same __ubsan_handle_type_mismatch_v1
same __ubsan_handle_type_mismatch_v1_abort
same __ubsan_handle_vla_bound_not_positive
same __ubsan_handle_vla_bound_not_positive_abort
